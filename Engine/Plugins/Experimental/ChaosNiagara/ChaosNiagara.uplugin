{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Niagara",
	"Description": "Import destruction data from Chaos into Niagara to generate secondary destruction effects.",
	"Category": "",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosNiagara",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Niagara",
			"Enabled": true
		},
		{
			"Name": "ChaosSolverPlugin",
			"Enabled": true
		}
	]
}
