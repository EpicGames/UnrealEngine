// Copyright Epic Games, Inc. All Rights Reserved.

Buffer<float3> {ParameterName}_PositionBuffer;
Buffer<float3> {ParameterName}_VelocityBuffer;
Buffer<float>  {ParameterName}_ExtentMinBuffer;
Buffer<float>  {ParameterName}_ExtentMaxBuffer;
Buffer<float>  {ParameterName}_VolumeBuffer;
Buffer<int>    {ParameterName}_SolverIDBuffer; // NOTE(mv): Not used?
Buffer<float>  {ParameterName}_DensityBuffer;
Buffer<float>  {ParameterName}_FrictionBuffer;
Buffer<float>  {ParameterName}_RestitutionBuffer;
Buffer<int>    {ParameterName}_SurfaceTypeBuffer;
Buffer<float4> {ParameterName}_ColorBuffer;
		
Buffer<float3> {ParameterName}_IncomingLocationBuffer;
Buffer<float3> {ParameterName}_IncomingAccumulatedImpulseBuffer;
Buffer<float3> {ParameterName}_IncomingNormalBuffer;
Buffer<float3> {ParameterName}_IncomingVelocity1Buffer;
Buffer<float3> {ParameterName}_IncomingVelocity2Buffer;
Buffer<float3> {ParameterName}_IncomingAngularVelocity1Buffer;
Buffer<float3> {ParameterName}_IncomingAngularVelocity2Buffer;
Buffer<float>  {ParameterName}_IncomingMass1Buffer;
Buffer<float>  {ParameterName}_IncomingMass2Buffer;
Buffer<float>  {ParameterName}_IncomingTimeBuffer;

Buffer<float3> {ParameterName}_TransformTranslationBuffer;
Buffer<float4> {ParameterName}_TransformRotationBuffer;
Buffer<float3> {ParameterName}_TransformScaleBuffer;
Buffer<float3> {ParameterName}_BoundsBuffer;

// NOTE(mv): Not implemented in the CPU-side functionality yet. 
//           Returns 0 in GetPointType instead.
//           
// Buffer<int> PointTypeBuffer;

int {ParameterName}_LastSpawnedPointID;

float {ParameterName}_SolverTime;


void GetPosition_{ParameterName}(in int ParticleID, out float3 Out_Position) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Position = {ParameterName}_PositionBuffer[ParticleID];
}

void GetNormal_{ParameterName}(in int ParticleID, out float3 Out_Normal)
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Normal = {ParameterName}_IncomingNormalBuffer[ParticleID];
}

void GetVelocity_{ParameterName}(in int ParticleID, out float3 Out_Velocity) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Velocity = {ParameterName}_VelocityBuffer[ParticleID];
}

void GetAngularVelocity_{ParameterName}(in int ParticleID, out float3 Out_AngularVelocity) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_AngularVelocity = {ParameterName}_IncomingAngularVelocity1Buffer[ParticleID];
}

void GetExtentMin_{ParameterName}(in int ParticleID, out float Out_ExtentMin) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_ExtentMin = {ParameterName}_ExtentMinBuffer[ParticleID];
}

void GetExtentMax_{ParameterName}(in int ParticleID, out float Out_ExtentMax) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_ExtentMax = {ParameterName}_ExtentMaxBuffer[ParticleID];
}

void GetVolume_{ParameterName}(in int ParticleID, out float Out_Volume) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Volume = {ParameterName}_VolumeBuffer[ParticleID];
}

void GetParticleIdsToSpawnAtTime_{ParameterName}(in float Time, out int Out_Min, out int Out_Max, out int Out_Count) 
{
	// This function cannot be called on the GPU, as all spawn scripts are run on the CPU..
	// TODO: Find a way to warn/error about this.
	Out_Count = 0;
	Out_Min = 0;
	Out_Max = 0;
}

void GetPointType_{ParameterName}(in int ParticleID, out int Out_PointType) 
{
	// NOTE(mv): Not yet part of the CPU functionality.
	Out_PointType = 0;
}

void GetColor_{ParameterName}(in int ParticleID, out float4 Out_Color) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Color = {ParameterName}_ColorBuffer[ParticleID];
}

void GetSolverTime_{ParameterName}(out float Out_SolverTime) 
{
	Out_SolverTime = {ParameterName}_SolverTime;
}

void GetDensity_{ParameterName}(in int ParticleID, out float Out_Density) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Density = {ParameterName}_DensityBuffer[ParticleID];
}

void GetFriction_{ParameterName}(in int ParticleID, out float Out_Friction) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Friction = {ParameterName}_FrictionBuffer[ParticleID];
}

void GetRestitution_{ParameterName}(in int ParticleID, out float Out_Restitution) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Restitution = {ParameterName}_RestitutionBuffer[ParticleID];
}

void GetTransform_{ParameterName}(in int ParticleID, out float3 Out_Translation, out float4 Out_Rotation, out float3 Out_Scale) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Translation = {ParameterName}_TransformTranslationBuffer[ParticleID];
	Out_Rotation = {ParameterName}_TransformRotationBuffer[ParticleID];
	Out_Scale = {ParameterName}_TransformScaleBuffer[ParticleID];
}

void GetSize_{ParameterName}(in int ParticleID, out float3 Out_Size) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Size = {ParameterName}_BoundsBuffer[ParticleID];
}

void GetSurfaceType_{ParameterName}(in int ParticleID, out int Out_SurfaceType) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_SurfaceType = {ParameterName}_SurfaceTypeBuffer[ParticleID];
}

void GetCollisionData_{ParameterName}(in int ParticleID, out float3 Out_Location, out float3 Out_AccumulatedImpulse, out float3 Out_Normal, out float3 Out_Velocity1, out float3 Out_Velocity2, out float3 Out_AngularVelocity1, out float3 Out_AngularVelocity2, out float Out_Mass1, out float Out_Mass2, out float Out_Time) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Location = {ParameterName}_IncomingLocationBuffer[ParticleID];
	Out_AccumulatedImpulse = {ParameterName}_IncomingAccumulatedImpulseBuffer[ParticleID];
	Out_Normal = {ParameterName}_IncomingNormalBuffer[ParticleID];
	Out_Velocity1 = {ParameterName}_IncomingVelocity1Buffer[ParticleID];
	Out_Velocity2 = {ParameterName}_IncomingVelocity2Buffer[ParticleID];
	Out_AngularVelocity1 = {ParameterName}_IncomingAngularVelocity1Buffer[ParticleID];
	Out_AngularVelocity2 = {ParameterName}_IncomingAngularVelocity2Buffer[ParticleID];
	Out_Mass1 = {ParameterName}_IncomingMass1Buffer[ParticleID];
	Out_Mass2 = {ParameterName}_IncomingMass2Buffer[ParticleID];
	Out_Time = {ParameterName}_IncomingTimeBuffer[ParticleID];
}

void GetBreakingData_{ParameterName}(in int ParticleID, out float3 Out_Location, out float3 Out_Velocity, out float3 Out_AngularVelocity, out float Out_Mass, out float Out_Time) 
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Location = {ParameterName}_IncomingLocationBuffer[ParticleID];
	Out_Velocity = {ParameterName}_IncomingVelocity1Buffer[ParticleID];
	Out_AngularVelocity = {ParameterName}_IncomingAngularVelocity1Buffer[ParticleID];
	Out_Mass = {ParameterName}_IncomingMass1Buffer[ParticleID];
	Out_Time = {ParameterName}_IncomingTimeBuffer[ParticleID];
}

void GetTrailingData_{ParameterName}(in int ParticleID, out float3 Out_Location, out float3 Out_Velocity, out float3 Out_AngularVelocity, out float Out_Mass, out float Out_Time)
{
	ParticleID -= {ParameterName}_LastSpawnedPointID + 1;
	Out_Location = {ParameterName}_IncomingLocationBuffer[ParticleID];
	Out_Velocity = {ParameterName}_IncomingVelocity1Buffer[ParticleID];
	Out_AngularVelocity = {ParameterName}_IncomingAngularVelocity1Buffer[ParticleID];
	Out_Mass = {ParameterName}_IncomingMass1Buffer[ParticleID];
	Out_Time = {ParameterName}_IncomingTimeBuffer[ParticleID];
}
