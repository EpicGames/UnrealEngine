// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfaceGeometryCollection.ush 
=============================================================================*/

#include "/Plugin/FX/Niagara/Private/NiagaraTransformUtils.ush"

/* -----------------------------------------------------------------
 * static mesh constants and context
 * -----------------------------------------------------------------
 */

#define BOX_INDEX 0
#define SPHERE_INDEX 1
#define CAPSULE_INDEX 2

float3			{ParameterName}_BoundsMin;
float3			{ParameterName}_BoundsMax;
int				{ParameterName}_NumPieces;
float3			{ParameterName}_RootTransform_Translation;
float4			{ParameterName}_RootTransform_Rotation;
float3			{ParameterName}_RootTransform_Scale;
Buffer<float4>	{ParameterName}_WorldTransformBuffer;
Buffer<float4>	{ParameterName}_PrevWorldTransformBuffer;
Buffer<float4>	{ParameterName}_WorldInverseTransformBuffer;
Buffer<float4>	{ParameterName}_PrevWorldInverseTransformBuffer;
Buffer<float4>	{ParameterName}_BoundsBuffer;
ByteAddressBuffer  {ParameterName}_ElementTransforms;

/* -----------------------------------------------------------------
 * Position / Transform utils
 * -----------------------------------------------------------------
 */

float3x4 DIGeometryCollection_GetCurrentTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex;
    return float3x4({ParameterName}_WorldTransformBuffer[BufferOffset], {ParameterName}_WorldTransformBuffer[BufferOffset+1], {ParameterName}_WorldTransformBuffer[BufferOffset+2]);
}

float3x4 DIGeometryCollection_GetInverseTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex;
    return float3x4({ParameterName}_WorldInverseTransformBuffer[BufferOffset], {ParameterName}_WorldInverseTransformBuffer[BufferOffset+1], {ParameterName}_WorldInverseTransformBuffer[BufferOffset+2]);
}

float3x4 DIGeometryCollection_GetPreviousTransform_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex;
    return float3x4({ParameterName}_PrevWorldTransformBuffer[BufferOffset], {ParameterName}_PrevWorldTransformBuffer[BufferOffset+1], {ParameterName}_PrevWorldTransformBuffer[BufferOffset+2]);
}

float3x4 DIGeometryCollection_GetPreviousInverse_{ParameterName}(in int ElementIndex)
{
	const int BufferOffset = 3 * ElementIndex;
    return float3x4({ParameterName}_PrevWorldInverseTransformBuffer[BufferOffset], {ParameterName}_PrevWorldInverseTransformBuffer[BufferOffset+1], {ParameterName}_PrevWorldInverseTransformBuffer[BufferOffset+2]);
}

/* -----------------------------------------------------------------
 * Collision detection utils
 * -----------------------------------------------------------------
 */

// Given a world space position (WorldPosition) compute the sphere closest point (position,normal,velocity)
float DIGeometryCollection_GetBoxProjection_{ParameterName}(in float3 LocalPosition, in float3 BoxExtent, in int BoxIndex, 
			inout float3 OutClosestPosition, inout float3 OutClosestNormal, inout int OutElementIndex, inout float OutMinDistance)
{
	const float3 HalfExtent = 0.5 * BoxExtent;
	const float3 DeltaPosition = abs(LocalPosition) - HalfExtent;

	const int ClosestAxis = ((DeltaPosition.x > DeltaPosition.y) && (DeltaPosition.x > DeltaPosition.z)) ? 0 : ( DeltaPosition.y > DeltaPosition.z) ? 1 : 2;
	const float OutsideDistance = length(max(DeltaPosition,0.0));
	const float BoxDistance = OutsideDistance + min(DeltaPosition[ClosestAxis],0.0);

	if (BoxDistance < OutMinDistance)
	{
		OutMinDistance = BoxDistance;
		OutElementIndex = BoxIndex;

		if (BoxDistance <= 0)
		{
			const bool NegativeSide = LocalPosition[ClosestAxis] < 0.0;

			OutClosestPosition = LocalPosition;
			OutClosestNormal = float3(0,0,0);
			if( ClosestAxis == 0) 
			{
				OutClosestPosition.x = NegativeSide ? -HalfExtent.x : HalfExtent.x;
				OutClosestNormal.x = NegativeSide ? -1.0 : 1.0;
			}
			else if( ClosestAxis == 1) 
			{
				OutClosestPosition.y = NegativeSide ? -HalfExtent.y : HalfExtent.y;
				OutClosestNormal.y = NegativeSide ? -1.0 : 1.0;
			}
			else if( ClosestAxis == 2) 
			{
				OutClosestPosition.z = NegativeSide ? -HalfExtent.z : HalfExtent.z;
				OutClosestNormal.z = NegativeSide ? -1.0 : 1.0;
			}
		}
		else
		{
			OutClosestPosition = clamp(LocalPosition,-HalfExtent,HalfExtent);
			OutClosestNormal = (LocalPosition - OutClosestPosition) / OutsideDistance;
		}
	}
	return BoxDistance;
}

void GetNumElements_{ParameterName}(out int OutCount)
{
	OutCount = {ParameterName}_NumPieces;
}

void GetGeometryComponentTransform_{ParameterName}(out float3 OutPosition, out float4 OutRotation, out float3 OutScale)
{
	OutPosition = {ParameterName}_RootTransform_Translation;
	OutRotation = {ParameterName}_RootTransform_Rotation;
	OutScale = {ParameterName}_RootTransform_Scale;
}

void GetElementTransform_{ParameterName}(in int InElementIndex, out float3 OutTranslation, out float4 OutRotation, out float3 OutScale)
{
	OutTranslation = 0;
	OutRotation = 0;
	OutScale = 0;
	if (InElementIndex >= 0 && InElementIndex < {ParameterName}_NumPieces)
	{
		uint TransformIndex = InElementIndex * 10 * 4;
		FNiagaraTransform ElementTransform = LoadTransform({ParameterName}_ElementTransforms, TransformIndex);
		OutTranslation = ElementTransform.Translation;
		OutRotation = ElementTransform.Rotation;
		OutScale = ElementTransform.Scale;
	}
}

void GetElementBounds_{ParameterName}(in int InElementIndex, out float3 OutCenter, out float3 OutBoundingBox)
{
	if (InElementIndex >= 0 && InElementIndex < {ParameterName}_NumPieces)
	{
		uint TransformIndex = InElementIndex * 10 * 4;
		FNiagaraTransform ElementTransform = LoadTransform({ParameterName}_ElementTransforms, TransformIndex);
		OutCenter = ElementTransform.Translation;
		OutBoundingBox = {ParameterName}_BoundsBuffer[InElementIndex].xyz;
	}
	else
	{
		OutCenter = 0;
		OutBoundingBox = 0;
	}
}

/* -----------------------------------------------------------------
 * Get the closest element to the world position
 * -----------------------------------------------------------------
 */

float3 DIGeometryCollection_GetLocalPosition_{ParameterName}(in float3 WorldPosition, in int ElementIndex, in float TimeFraction )
{
	const float3 CurrentLocal = mul(DIGeometryCollection_GetInverseTransform_{ParameterName}(ElementIndex), float4(WorldPosition,1.0)).xyz;
	const float3 PreviousLocal = mul(DIGeometryCollection_GetPreviousInverse_{ParameterName}(ElementIndex), float4(WorldPosition,1.0)).xyz;

	return PreviousLocal + TimeFraction * (CurrentLocal-PreviousLocal);
}

// Given a world space position (WorldPosition) compute the static mesh closest point (position,normal,velocity)
void DIGeometryCollection_GetClosestPointNoNormal_{ParameterName}(in float3 WorldPosition, in float DeltaTime, in float TimeFraction, out float OutClosestDistance, out float3 OutClosestPosition, out float3 OutClosestVelocity, out int ElementIndex)
{	
	float3 CollisionPosition = float3(0,0,0);
	float3 CollisionNormal = float3(0,0,0);
	OutClosestDistance = MAX_DISTANCE;	
	
	OutClosestPosition = float3(0,0,0);
	OutClosestVelocity = float3(0,0,0);
	ElementIndex = -1;
	
	[branch]
	if (WorldPosition.x < {ParameterName}_BoundsMin.x || 
		WorldPosition.y < {ParameterName}_BoundsMin.y || 
		WorldPosition.z < {ParameterName}_BoundsMin.z || 
		WorldPosition.x > {ParameterName}_BoundsMax.x || 
		WorldPosition.y > {ParameterName}_BoundsMax.y || 
		WorldPosition.z > {ParameterName}_BoundsMax.z)
	{
		return;
	}
	

	int ClosestTransform = -1;
	for (int CurrTransform = 0; CurrTransform < {ParameterName}_NumPieces; ++CurrTransform)
	{
		float3 LocalPosition = DIGeometryCollection_GetLocalPosition_{ParameterName}(WorldPosition,CurrTransform,TimeFraction);
		
		DIGeometryCollection_GetBoxProjection_{ParameterName}(LocalPosition, {ParameterName}_BoundsBuffer[CurrTransform].xyz, CurrTransform,
				CollisionPosition, CollisionNormal, ClosestTransform, OutClosestDistance);
	}

	if (ClosestTransform != -1)
	{
		const float3 PreviousPosition = mul(DIGeometryCollection_GetPreviousTransform_{ParameterName}(ClosestTransform), float4(CollisionPosition,1.0)).xyz;
		const float3 CurrentPosition = mul(DIGeometryCollection_GetCurrentTransform_{ParameterName}(ClosestTransform), float4(CollisionPosition,1.0)).xyz;
		OutClosestVelocity = ( CurrentPosition - PreviousPosition ) / DeltaTime;
		OutClosestPosition = PreviousPosition + TimeFraction * (CurrentPosition-PreviousPosition);
		ElementIndex = ClosestTransform;
	}
}


void GetClosestPointNoNormal_{ParameterName}(in float3 WorldPosition, in float DeltaTime, in float TimeFraction, out float ClosestDistance, out float3 OutClosestPosition, out float3 OutClosestVelocity, out int ElementIndex)
{
	DIGeometryCollection_GetClosestPointNoNormal_{ParameterName}(WorldPosition,DeltaTime, TimeFraction, ClosestDistance, OutClosestPosition, OutClosestVelocity, ElementIndex);
}

#undef BOX_INDEX
#undef SPHERE_INDEX
#undef CAPSULE_INDEX
