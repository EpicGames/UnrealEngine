// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
NiagaraDataInterfacePhysicsField.ush 
=============================================================================*/

/* -----------------------------------------------------------------
 * Field System constants and context
 * -----------------------------------------------------------------
 */

#include "/Engine/Private/PhysicsFieldShared.ush"
#include "/Engine/Private/PhysicsFieldEval.ush"

// Global param for the world field
int4			{ParameterName}_TargetMappings[MAX_PHYSICS_FIELD_TARGETS];  
int				{ParameterName}_bClipmapAvailable;

Buffer<float>	{ParameterName}_ClipmapBuffer;
float3			{ParameterName}_ClipmapCenter;
float			{ParameterName}_ClipmapDistance;
int				{ParameterName}_ClipmapResolution;
int				{ParameterName}_ClipmapExponent;
int				{ParameterName}_ClipmapCount;
int				{ParameterName}_TargetCount;
int4			{ParameterName}_FieldTargets[MAX_PHYSICS_FIELD_TARGETS];

Buffer<float>	{ParameterName}_NodesParams;
Buffer<int>	{ParameterName}_NodesOffsets;
Buffer<int>	{ParameterName}_TargetsOffsets;
float {ParameterName}_TimeSeconds;

float3 PhysicsVectorField_{ParameterName}(in float3 WorldPosition, in int TargetIndex, in bool bUseClipmap)
{
	// LWC_TODO: fix this when the physics field functions are lwc ready
	FLWCVector3 SamplePos = MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition);

	if ( {ParameterName}_bClipmapAvailable && bUseClipmap )
	{
		return PhysicsField_SamplePhysicsVectorField(
				LWCHackToFloat(SamplePos), TargetIndex, {ParameterName}_FieldTargets, {ParameterName}_TargetCount, {ParameterName}_ClipmapCenter, {ParameterName}_ClipmapDistance, 
				{ParameterName}_ClipmapExponent, {ParameterName}_ClipmapCount, {ParameterName}_ClipmapResolution, {ParameterName}_ClipmapBuffer
			);
	}
	else
	{
		return PhysicsField_EvalPhysicsVectorField(LWCHackToFloat(SamplePos), {ParameterName}_TargetMappings[TargetIndex].x, {ParameterName}_NodesParams, {ParameterName}_NodesOffsets, {ParameterName}_TargetsOffsets, {ParameterName}_TimeSeconds);
	}
}

float PhysicsScalarField_{ParameterName}(in float3 WorldPosition, in int TargetIndex, in bool bUseClipmap)
{
	// LWC_TODO: fix this when the physics field functions are lwc ready
	FLWCVector3 SamplePos = MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition);

	if ( {ParameterName}_bClipmapAvailable && bUseClipmap )
	{
		return PhysicsField_SamplePhysicsScalarField(
					LWCHackToFloat(SamplePos), TargetIndex, {ParameterName}_FieldTargets, {ParameterName}_TargetCount, {ParameterName}_ClipmapCenter, {ParameterName}_ClipmapDistance,
					{ParameterName}_ClipmapExponent, {ParameterName}_ClipmapCount, {ParameterName}_ClipmapResolution, {ParameterName}_ClipmapBuffer
				);
	}
	else
	{
		return PhysicsField_EvalPhysicsScalarField(LWCHackToFloat(SamplePos), {ParameterName}_TargetMappings[TargetIndex].y, {ParameterName}_NodesParams, {ParameterName}_NodesOffsets, {ParameterName}_TargetsOffsets, {ParameterName}_TimeSeconds);
	}
}

int PhysicsIntegerField_{ParameterName}(in float3 WorldPosition, in int TargetIndex, in bool bUseClipmap)
{
	// LWC_TODO: fix this when the physics field functions are lwc ready
	FLWCVector3 SamplePos = MakeLWCVector3(GetEngineOwnerLWCTile(), WorldPosition);

	if ( {ParameterName}_bClipmapAvailable && bUseClipmap )
	{
		return PhysicsField_SamplePhysicsIntegerField(
					LWCHackToFloat(SamplePos), TargetIndex, {ParameterName}_FieldTargets, {ParameterName}_TargetCount, {ParameterName}_ClipmapCenter, {ParameterName}_ClipmapDistance,
					{ParameterName}_ClipmapExponent, {ParameterName}_ClipmapCount, {ParameterName}_ClipmapResolution, {ParameterName}_ClipmapBuffer
				);
	}
	else
	{
		return PhysicsField_EvalPhysicsIntegerField(LWCHackToFloat(SamplePos), {ParameterName}_TargetMappings[TargetIndex].z, {ParameterName}_NodesParams, {ParameterName}_NodesOffsets, {ParameterName}_TargetsOffsets, {ParameterName}_TimeSeconds);
	}
}

void SamplePhysicsVectorField_{ParameterName}(in float3 WorldPosition, in int TargetIndex, out float3 OutFieldVector)
{
	OutFieldVector = PhysicsVectorField_{ParameterName}(WorldPosition, TargetIndex, true);
}

void SamplePhysicsScalarField_{ParameterName}(in float3 WorldPosition, in int TargetIndex, out float OutFieldScalar)
{
	OutFieldScalar = PhysicsScalarField_{ParameterName}(WorldPosition, TargetIndex, true);
}

void SamplePhysicsIntegerField_{ParameterName}(in float3 WorldPosition, in int TargetIndex, out int OutFieldInteger)
{
	OutFieldInteger = PhysicsIntegerField_{ParameterName}(WorldPosition, TargetIndex, true);
}

void EvalPhysicsVectorField_{ParameterName}(in float3 WorldPosition, in int TargetIndex, out float3 OutFieldVector)
{
	OutFieldVector = PhysicsVectorField_{ParameterName}(WorldPosition, TargetIndex, false);
}

void EvalPhysicsScalarField_{ParameterName}(in float3 WorldPosition, in int TargetIndex, out float OutFieldScalar)
{
	OutFieldScalar = PhysicsScalarField_{ParameterName}(WorldPosition, TargetIndex, false);
}

void EvalPhysicsIntegerField_{ParameterName}(in float3 WorldPosition, in int TargetIndex, out int OutFieldInteger)
{
	OutFieldInteger = PhysicsIntegerField_{ParameterName}(WorldPosition, TargetIndex, false);
}

void GetPhysicsFieldResolution_{ParameterName}(out float3 OutTextureSize)
{
	OutTextureSize = {ParameterName}_ClipmapResolution;
}

void GetPhysicsFieldBounds_{ParameterName}(out float3 OutMinBounds, out float3 OutMaxBounds)
{
	OutMinBounds = {ParameterName}_ClipmapCenter - {ParameterName}_ClipmapDistance;
	OutMaxBounds = {ParameterName}_ClipmapCenter + {ParameterName}_ClipmapDistance;
}
