// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class ChaosNiagara : ModuleRules
{
	public ChaosNiagara(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePaths.AddRange(new string[] {
		});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"Projects",
				"Slate",
				"SlateCore",
				"RenderCore",
                "Chaos",
				"PhysicsCore",
			}
        );
					
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
              	"NiagaraCore",
				"Niagara",
				"NiagaraShader",
				"CoreUObject",
				"VectorVM",
				"RHI",
				"NiagaraVertexFactories",
                "ChaosSolverEngine",
                "GeometryCollectionEngine",
				"FieldSystemEngine"
            }
        );

		PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
	}
}
