// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Declares.h"
#include "NiagaraDataInterface.h"
#include "Chaos/ExternalCollisionData.h"
#include "RenderGraphFwd.h"
#include "NiagaraDataInterfaceChaosDestruction.generated.h"

#define UE_API CHAOSNIAGARA_API

class AChaosSolverActor;
class FPhysScene_Chaos;
class UGeometryCollectionComponent;
namespace Chaos { struct FBreakingEventData; }
namespace Chaos { struct FCollisionEventData; }
namespace Chaos { struct FTrailingEventData; }
struct FNiagaraDataInterfaceGeneratedFunction;

struct PhysicsProxyWrapper;

USTRUCT()
struct FChaosDestructionEvent
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Chaos Destruction Event")
	FVector Position;

	UPROPERTY(EditAnywhere, Category = "Chaos Destruction Event")
	FVector Normal;

	UPROPERTY(EditAnywhere, Category = "Chaos Destruction Event")
	FVector Velocity;

	UPROPERTY(EditAnywhere, Category = "Chaos Destruction Event")
	FVector AngularVelocity;

	UPROPERTY(EditAnywhere, Category = "Chaos Destruction Event")
	float ExtentMin;

	UPROPERTY(EditAnywhere, Category = "Chaos Destruction Event")
	float ExtentMax;

	UPROPERTY(EditAnywhere, Category = "Chaos Destruction Event")
	int32 ParticleID;

	UPROPERTY(EditAnywhere, Category = "Chaos Destruction Event")
	float Time;

	UPROPERTY(EditAnywhere, Category = "Chaos Destruction Event")
	int32 Type;

	FChaosDestructionEvent()
		: Position(FVector::ZeroVector)
		, Normal(FVector::ZeroVector)
		, Velocity(FVector::ZeroVector)
		, AngularVelocity(FVector::ZeroVector)
		, ExtentMin(0.f)
		, ExtentMax(0.f)
		, ParticleID()
		, Time(0.f)
		, Type(-1)
	{
	}

	inline bool operator==(const FChaosDestructionEvent& Other) const
	{
		if ((Other.Position != Position) || (Other.Normal != Normal)
			|| (Other.Velocity != Velocity)
			|| (Other.AngularVelocity != AngularVelocity)
			|| (Other.ExtentMin != ExtentMin)
			|| (Other.ExtentMax != ExtentMax)
			|| (Other.ParticleID != ParticleID) || (Other.Time != Time)
			|| (Other.Type != Type))
			return false;

		return true;
	}

	inline bool operator!=(const FChaosDestructionEvent& Other) const
	{
		return !(*this == Other);
	}
};

struct FSolverData
{
	FSolverData()
		: Solver(nullptr)
	{}

	FPhysScene_Chaos* PhysScene;
	Chaos::FPhysicsSolver* Solver;
};

struct FNDIChaosDestruction_InstanceData
{
	TArray<FVector> PositionArray;
	TArray<FVector> VelocityArray;
	TArray<float> ExtentMinArray;
	TArray<float> ExtentMaxArray;
	TArray<float> VolumeArray;
	TArray<int32> SolverIDArray;
	TArray<float> DensityArray;
	TArray<float> FrictionArray;
	TArray<float> RestitutionArray;
	TArray<int32> SurfaceTypeArray;
	TArray<FLinearColor> ColorArray;

	TArray<FVector> IncomingLocationArray;				// Collision, Breaking, Trailing
	TArray<FVector> IncomingAccumulatedImpulseArray;	// Collision
	TArray<FVector> IncomingNormalArray;				// Collision
	TArray<FVector> IncomingVelocity1Array;				// Collision, Breaking, Trailing
	TArray<FVector> IncomingVelocity2Array;				// Collision
	TArray<FVector> IncomingAngularVelocity1Array;		// Collision, Breaking, Trailing
	TArray<FVector> IncomingAngularVelocity2Array;		// Collision
	TArray<float> IncomingMass1Array;					// Collision, Breaking, Trailing
	TArray<float> IncomingMass2Array;					// Collision
	TArray<float> IncomingTimeArray;					// Collision, Breaking, Trailing

	TArray<FVector> TransformTranslationArray;
	TArray<FQuat> TransformRotationArray;
	TArray<FVector> TransformScaleArray;
	TArray<FVector> BoundsArray;
};

struct FNiagaraDIChaosDestruction_GPUData
{
	TArray<FVector3f> PositionArray;
	TArray<FVector3f> VelocityArray;
	TArray<float> ExtentMinArray;
	TArray<float> ExtentMaxArray;
	TArray<float> VolumeArray;
	TArray<int32> SolverIDArray;
	TArray<float> DensityArray;
	TArray<float> FrictionArray;
	TArray<float> RestitutionArray;
	TArray<int32> SurfaceTypeArray;
	TArray<FLinearColor> ColorArray;

	TArray<FVector3f> IncomingLocationArray;				// Collision, Breaking, Trailing
	TArray<FVector3f> IncomingAccumulatedImpulseArray;	// Collision
	TArray<FVector3f> IncomingNormalArray;				// Collision
	TArray<FVector3f> IncomingVelocity1Array;			// Collision, Breaking, Trailing
	TArray<FVector3f> IncomingVelocity2Array;			// Collision
	TArray<FVector3f> IncomingAngularVelocity1Array;		// Collision, Breaking, Trailing
	TArray<FVector3f> IncomingAngularVelocity2Array;		// Collision
	TArray<float> IncomingMass1Array;					// Collision, Breaking, Trailing
	TArray<float> IncomingMass2Array;					// Collision
	TArray<float> IncomingTimeArray;					// Collision, Breaking, Trailing

	TArray<FVector3f> TransformTranslationArray;
	TArray<FQuat4f> TransformRotationArray;
	TArray<FVector3f> TransformScaleArray;
	TArray<FVector3f> BoundsArray;

	void ResetAll()
	{
		PositionArray.Reset();
		VelocityArray.Reset();
		ExtentMinArray.Reset();
		ExtentMaxArray.Reset();
		VolumeArray.Reset();
		SolverIDArray.Reset();
		DensityArray.Reset();
		FrictionArray.Reset();
		RestitutionArray.Reset();
		SurfaceTypeArray.Reset();
		ColorArray.Reset();

		IncomingLocationArray.Reset();
		IncomingAccumulatedImpulseArray.Reset();
		IncomingNormalArray.Reset();
		IncomingVelocity1Array.Reset();
		IncomingVelocity2Array.Reset();
		IncomingAngularVelocity1Array.Reset();
		IncomingAngularVelocity2Array.Reset();
		IncomingMass1Array.Reset();
		IncomingMass2Array.Reset();
		IncomingTimeArray.Reset();
		
		TransformTranslationArray.Reset();
		TransformRotationArray.Reset();
		TransformScaleArray.Reset();
		BoundsArray.Reset();
	}

	float SolverTime;
	int32 LastSpawnedPointID;

	FRDGBufferSRVRef RDGPositionSRV = nullptr;
	FRDGBufferSRVRef RDGVelocitySRV = nullptr;
	FRDGBufferSRVRef RDGExtentMinBufferSRV = nullptr;
	FRDGBufferSRVRef RDGExtentMaxBufferSRV = nullptr;
	FRDGBufferSRVRef RDGVolumeBufferSRV = nullptr;
	FRDGBufferSRVRef RDGSolverIDBufferSRV = nullptr;
	FRDGBufferSRVRef RDGDensityBufferSRV = nullptr;
	FRDGBufferSRVRef RDGFrictionBufferSRV = nullptr;
	FRDGBufferSRVRef RDGRestitutionBufferSRV = nullptr;
	FRDGBufferSRVRef RDGSurfaceTypeBufferSRV = nullptr;
	FRDGBufferSRVRef RDGColorBufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingLocationBufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingAccumulatedImpulseBufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingNormalBufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingVelocity1BufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingVelocity2BufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingAngularVelocity1BufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingAngularVelocity2BufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingMass1BufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingMass2BufferSRV = nullptr;
	FRDGBufferSRVRef RDGIncomingTimeBufferSRV = nullptr;
	FRDGBufferSRVRef RDGTransformTranslationBufferSRV = nullptr;
	FRDGBufferSRVRef RDGTransformRotationBufferSRV = nullptr;
	FRDGBufferSRVRef RDGTransformScaleBufferSRV = nullptr;
	FRDGBufferSRVRef RDGBoundsBufferSRV = nullptr;
};

struct FNiagaraDIChaosDestruction_InstanceDataToPassToRT
{
	TArray<FVector3f> PositionArray;
	TArray<FVector3f> VelocityArray;
	TArray<float> ExtentMinArray;
	TArray<float> ExtentMaxArray;
	TArray<float> VolumeArray;
	TArray<int32> SolverIDArray;
	TArray<float> DensityArray;
	TArray<float> FrictionArray;
	TArray<float> RestitutionArray;
	TArray<int32> SurfaceTypeArray;
	TArray<FLinearColor> ColorArray;

	TArray<FVector3f> IncomingLocationArray;				// Collision, Breaking, Trailing
	TArray<FVector3f> IncomingAccumulatedImpulseArray;	// Collision
	TArray<FVector3f> IncomingNormalArray;				// Collision
	TArray<FVector3f> IncomingVelocity1Array;			// Collision, Breaking, Trailing
	TArray<FVector3f> IncomingVelocity2Array;			// Collision
	TArray<FVector3f> IncomingAngularVelocity1Array;		// Collision, Breaking, Trailing
	TArray<FVector3f> IncomingAngularVelocity2Array;		// Collision
	TArray<float> IncomingMass1Array;					// Collision, Breaking, Trailing
	TArray<float> IncomingMass2Array;					// Collision
	TArray<float> IncomingTimeArray;					// Collision, Breaking, Trailing
	
	TArray<FVector3f> TransformTranslationArray; // Breaking
	TArray<FQuat4f> TransformRotationArray; // Breaking
	TArray<FVector3f> TransformScaleArray; // Breaking
	TArray<FVector3f> BoundsArray; // Breaking

	float SolverTime;
	int32 LastSpawnedPointID;

	FNiagaraDIChaosDestruction_InstanceDataToPassToRT()
	{
		FMemory::Memzero(this, sizeof(FNiagaraDIChaosDestruction_InstanceDataToPassToRT));
	}
};

UENUM(BlueprintType)
enum class EDataSortTypeEnum : uint8
{
	ChaosNiagara_DataSortType_NoSorting UMETA(DisplayName = "No Sorting"),
	ChaosNiagara_DataSortType_RandomShuffle UMETA(DisplayName = "Random Shuffle"),
	ChaosNiagara_DataSortType_SortByMassMaxToMin UMETA(DisplayName = "Sort by Mass - Max to Min"),
	ChaosNiagara_DataSortType_SortByMassMinToMax UMETA(DisplayName = "Sort by Mass - Min to Max"),
	//~~~
	//256th entry
	ChaosNiagara_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ERandomVelocityGenerationTypeEnum : uint8
{
	ChaosNiagara_RandomVelocityGenerationType_RandomDistribution UMETA(DisplayName = "Random Distribution"),
	ChaosNiagara_RandomVelocityGenerationType_RandomDistributionWithStreamers UMETA(DisplayName = "Random Distribution with Streamers"),
	ChaosNiagara_RandomVelocityGenerationType_CollisionNormalBased UMETA(DisplayName = "Collision Normal Based (Collision Data Only)"),
//	ChaosNiagara_RandomVelocityGenerationType_NRandomSpread UMETA(DisplayName = "N Random Spread"),
	//~~~
	//256th entry
	ChaosNiagara_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EDebugTypeEnum : uint8
{
	ChaosNiagara_DebugType_NoDebug UMETA(DisplayName = "No Debug"),
	ChaosNiagara_DebugType_ColorBySolver UMETA(DisplayName = "Color by Solver"),
	ChaosNiagara_DebugType_ColorByParticleIndex UMETA(DisplayName = "Color by ParticleIndex"),
	//~~~
	//256th entry
	ChaosNiagara_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EDataSourceTypeEnum : uint8
{
	ChaosNiagara_DataSourceType_Collision UMETA(DisplayName = "Collision Data"),
	ChaosNiagara_DataSourceType_Breaking UMETA(DisplayName = "Breaking Data"),
	ChaosNiagara_DataSourceType_Trailing UMETA(DisplayName = "Trailing Data"),
	//~~~
	//256th entry
	ChaosNiagara_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ELocationFilteringModeEnum : uint8
{
	ChaosNiagara_LocationFilteringMode_Inclusive UMETA(DisplayName = "Inclusive"),
	ChaosNiagara_LocationFilteringMode_Exclusive UMETA(DisplayName = "Exclusive"),
	//~~~
	//256th entry
	ChaosNiagara_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ELocationXToSpawnEnum : uint8
{
	ChaosNiagara_LocationXToSpawn_None UMETA(DisplayName = "None"),
	ChaosNiagara_LocationXToSpawn_Min UMETA(DisplayName = "Min <= LocationX"),
	ChaosNiagara_LocationXToSpawn_Max UMETA(DisplayName = "LocationX <= Max"),
	ChaosNiagara_LocationXToSpawn_MinMax UMETA(DisplayName = "Min <= LocationX <= Max"),
	//~~~
	//256th entry
	ChaosNiagara_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ELocationYToSpawnEnum : uint8
{
	ChaosNiagara_LocationYToSpawn_None UMETA(DisplayName = "None"),
	ChaosNiagara_LocationYToSpawn_Min UMETA(DisplayName = "Min <= LocationY"),
	ChaosNiagara_LocationYToSpawn_Max UMETA(DisplayName = "LocationY <= Max"),
	ChaosNiagara_LocationYToSpawn_MinMax UMETA(DisplayName = "Min <= LocationY <= Max"),
	//~~~
	//256th entry
	ChaosNiagara_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ELocationZToSpawnEnum : uint8
{
	ChaosNiagara_LocationZToSpawn_None UMETA(DisplayName = "None"),
	ChaosNiagara_LocationZToSpawn_Min UMETA(DisplayName = "Min <= LocationZ"),
	ChaosNiagara_LocationZToSpawn_Max UMETA(DisplayName = "LocationZ <= Max"),
	ChaosNiagara_LocationZToSpawn_MinMax UMETA(DisplayName = "Min <= LocationZ <= Max"),
	//~~~
	//256th entry
	ChaosNiagara_Max                UMETA(Hidden)
};

/** Data Interface allowing sampling of Chaos Destruction data. */
UCLASS(MinimalAPI, EditInlineNew, Category = "Chaos Niagara", meta = (DisplayName = "Chaos Destruction Data"))
class UNiagaraDataInterfaceChaosDestruction : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

public:
	/* Chaos Solver */
	UPROPERTY(EditAnywhere, Category = "Solver", meta = (DisplayName = "Chaos Solver"))
	TSet<TObjectPtr<AChaosSolverActor>> ChaosSolverActorSet;

	/* */
	UPROPERTY(EditAnywhere, Category = "Solver Data", meta = (DisplayName = "Data Source"))
	EDataSourceTypeEnum DataSourceType;

	/* Number of times the RBD collision data gets processed every second */
	UPROPERTY(EditAnywhere, Category = "Solver Data", meta = (DisplayName = "Data Process Frequency", UIMin = 0))
	int32 DataProcessFrequency;

	/* Maximum number of collision/breaking/trailing entry used for spawning particles every time data from the physics solver gets processed */
	UPROPERTY(EditAnywhere, Category = "Solver Data", meta = (DisplayName = "Max Number of Data to Spawn Particles", UIMin = 0))
	int32 MaxNumberOfDataEntriesToSpawn;

	/* Turn on/off particle spawning */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings", meta = (DisplayName = "Spawn Particles"))
	bool DoSpawn;

	/* For every collision random number of particles will be spawned in the range of [SpawnMultiplierMin, SpawnMultiplierMax]  */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings", meta = (DisplayName = "Spawn Count Min/Max", UIMin = 0))
	FVector2D SpawnMultiplierMinMax;

	/* For every collision random number of particles will be spawned in the range of [SpawnMultiplierMin, SpawnMultiplierMax]  */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings", meta = (DisplayName = "Chance to Spawn", UIMin = 0.f, UIMax = 1.f))
	float SpawnChance;

	/* Min/Max collision accumulated impulse to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max Collision Impulse To Spawn Particles", UIMin = 0.0))
	FVector2D ImpulseToSpawnMinMax;

	/* Min/Max speed to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max Speed To Spawn Particles", UIMin = 0.0))
	FVector2D SpeedToSpawnMinMax;

	/* Min/Max mass to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max Mass To Spawn Particles", UIMin = 0.0))
	FVector2D MassToSpawnMinMax;

	/* Min/Max ExtentMin to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max ExtentMin To Spawn Particles", UIMin = 0.0))
	FVector2D ExtentMinToSpawnMinMax;

	/* Min/Max ExtentMax to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max ExtentMax To Spawn Particles", UIMin = 0.0))
	FVector2D ExtentMaxToSpawnMinMax;

	/* Min/Max volume to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max Volume To Spawn Particles", UIMin = 0.0))
	FVector2D VolumeToSpawnMinMax;

	/* Min/Max solver time mass to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max SolverTime To Spawn Particles", UIMin = 0.0))
	FVector2D SolverTimeToSpawnMinMax;

	/* SurfaceType to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "SurfaceType To Spawn Particles", UIMin = 0.0))
	int32 SurfaceTypeToSpawn;

	/* Location Filtering Mode */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Location Filtering Mode"))
	ELocationFilteringModeEnum LocationFilteringMode;

	/* How to use LocationX to filter */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "LocationX To Spawn Particles"))
	ELocationXToSpawnEnum LocationXToSpawn;

	/* Min/Max LocationX to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max LocationX To Spawn Particles", UIMin = 0.0))
	FVector2D LocationXToSpawnMinMax;

	/* How to use LocationY to filter */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "LocationY To Spawn Particles"))
	ELocationYToSpawnEnum LocationYToSpawn;

	/* Min/Max LocationY to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max LocationY To Spawn Particles", UIMin = 0.0))
	FVector2D LocationYToSpawnMinMax;

	/* How to use LocationZ to filter */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "LocationZ To Spawn Particles"))
	ELocationZToSpawnEnum LocationZToSpawn;

	/* Min/Max LocationX to spawn particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Thresholds to Spawn", meta = (DisplayName = "Min/Max LocationZ To Spawn Particles", UIMin = 0.0))
	FVector2D LocationZToSpawnMinMax;

	/* Min Linear Speed to generate trailing particles */
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Trailing - Thresholds to Spawn", meta = (DisplayName = "Min Speed To Spawn Trailing Particles", UIMin = 0.0))
	float TrailMinSpeedToSpawn;

	/**
	* Sorting method to sort the collision data
	*/
	UPROPERTY(EditAnywhere, Category = "Spawn Settings - Sorting Solver Data", meta = (DisplayName = "Sorting Method"))
	EDataSortTypeEnum DataSortingType;

	/* TODO: Explanatory comment */
	UPROPERTY(VisibleAnywhere, Category = "Collision Data Settings", meta = (DisplayName = "Get External Collision Mesh and Physical Data"))
	bool bGetExternalCollisionData;

	/*  */
	UPROPERTY(EditAnywhere, Category = "Collision Data Settings", meta = (DisplayName = "Spatial Hash Collision/Breaking Data", UIMin = 0.0))
	bool DoSpatialHash;

	/* SpatialHash volume min */
	UPROPERTY(EditAnywhere, Category = "SpatialHash Settings", meta = (DisplayName = "SpatialHash Volume Min"))
	FVector SpatialHashVolumeMin;

	/* SpatialHash volume max */
	UPROPERTY(EditAnywhere, Category = "SpatialHash Settings", meta = (DisplayName = "SpatialHash Volume Max"))
	FVector SpatialHashVolumeMax;

	/* SpatialHash volume resolution */
	UPROPERTY(EditAnywhere, Category = "SpatialHash Settings", meta = (DisplayName = "SpatialHash Volume CellSize"))
	FVector SpatialHashVolumeCellSize;


	UPROPERTY(EditAnywhere, Category = "SpatialHash Settings", meta = (DisplayName = "Max Number of Collision/Breaking Per Cell", UIMin = 1))
	int32 MaxDataPerCell;

	/* Materials Filter */
	UPROPERTY(EditAnywhere, Category = "Data Settings", meta = (DisplayName = "Apply Materials Filter"))
	bool bApplyMaterialsFilter;

	/* TODO: Explanatory comment */
	UPROPERTY(EditAnywhere, Category = "Data Settings", meta = (DisplayName = "Filtered Materials", EditCondition = bApplyMaterialsFilter))
	TSet<TObjectPtr<UPhysicalMaterial>> ChaosBreakingMaterialSet;

	/* TODO: Explanatory comment */
	UPROPERTY(EditAnywhere, Category = "Data Settings", meta = (DisplayName = "Get External Breaking Mesh and Physical Data"))
	bool bGetExternalBreakingData;

	/* TODO: Explanatory comment */
	UPROPERTY(VisibleAnywhere, Category = "Data Settings", meta = (DisplayName = "Get External Trailing Mesh and Physical Data"))
	bool bGetExternalTrailingData;

	/* Random displacement value for the particle spawn position */
	UPROPERTY(EditAnywhere, Category = "Spawn Position Settings", meta = (DisplayName = "Position Random Spread Magnitude Min/Max", UIMin = 0.0))
	FVector2D RandomPositionMagnitudeMinMax;

	/* How much of the collision velocity gets inherited */
	UPROPERTY(EditAnywhere, Category = "Velocity Settings - Inherited Velocity", meta = (DisplayName = "Inherited Velocity Multiplier"))
	float InheritedVelocityMultiplier;

	/**
	* The method used to create the random velocities for the newly spawned particles
	*/
	UPROPERTY(EditAnywhere, Category = "Velocity Settings - Random Generation", meta = (DisplayName = "Velocity Model"))
	ERandomVelocityGenerationTypeEnum RandomVelocityGenerationType;

	/* Every particles will be spawned with random velocity with magnitude in the range of [RandomVelocityMagnitudeMin, RandomVelocityMagnitudeMax] */
	UPROPERTY(EditAnywhere, Category = "Velocity Settings", meta = (DisplayName = "Velocity Random Magnitude Min/Max", UIMin = 0.0))
	FVector2D RandomVelocityMagnitudeMinMax;

	/**/
	UPROPERTY(EditAnywhere, Category = "Velocity Settings - Collision Normal Based Velocity Model", meta = (DisplayName = "Spread Angle Max [Degrees]", UIMin = 0.0))
	float SpreadAngleMax;

	/* Min Offset value added to spawned particles velocity */
	UPROPERTY(EditAnywhere, Category = "Velocity Settings", meta = (DisplayName = "Velocity Offset Min"))
	FVector VelocityOffsetMin;

	/* Max Offset value added to spawned particles velocity */
	UPROPERTY(EditAnywhere, Category = "Velocity Settings", meta = (DisplayName = "Velocity Offset Max"))
	FVector VelocityOffsetMax;

	/* Clamp particles velocity */
	UPROPERTY(EditAnywhere, Category = "Velocity Settings", meta = (DisplayName = "Final Velocity Magnitude Maximum"))
	FVector2D FinalVelocityMagnitudeMinMax;

	UPROPERTY(EditAnywhere, Category = "Spawn Settings", meta = (DisplayName = "Max Latency"))
	float MaxLatency;

	/* Debug visualization method */
	UPROPERTY(EditAnywhere, Category = "Debug Settings", meta = (DisplayName = "Debug Visualization"))
	EDebugTypeEnum DebugType;

	/* ParticleIndex to process collisionData for */
//	UPROPERTY(EditAnywhere, Category = "Debug Settings", meta = (DisplayName = "ParticleIndex to Process"))
//	TGeometryParticleHandle<float, 3>* ParticleToProcess;

	//----------------------------------------------------------------------------
	// UObject Interface
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	//----------------------------------------------------------------------------
	// UNiagaraDataInterface Interface
	UE_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	UE_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	UE_API virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	UE_API virtual int32 PerInstanceDataSize() const override;

	UE_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc) override;
	UE_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true || (Target == ENiagaraSimTarget::CPUSim); }

	virtual bool HasPreSimulateTick() const override { return true; }
	//----------------------------------------------------------------------------
	// GPU sim functionality
#if WITH_EDITORONLY_DATA
	UE_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	UE_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	UE_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
#endif
	UE_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	UE_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	//----------------------------------------------------------------------------
	// EXPOSED FUNCTIONS
	template<typename ParticleIDParamType>
	void GetPosition(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetNormal(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetVelocity(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetAngularVelocity(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetExtentMin(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetExtentMax(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetVolume(FVectorVMExternalFunctionContext& Context);

	template<typename TimeParamType>
	void GetParticleIdsToSpawnAtTime(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetPointType(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetColor(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetSolverTime(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetDensity(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetFriction(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetRestitution(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetSize(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetTransform(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetSurfaceType(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetCollisionData(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetBreakingData(FVectorVMExternalFunctionContext& Context);

	template<typename ParticleIDParamType>
	void GetTrailingData(FVectorVMExternalFunctionContext& Context);

	// Sort predicates to sort data
	inline static bool CollisionDataSortByMassPredicateMaxToMin(const Chaos::FCollidingDataExt& Lhs, const Chaos::FCollidingDataExt& Rhs)
	{
		return FMath::Max(Lhs.Mass1, Lhs.Mass2) > FMath::Max(Rhs.Mass1, Rhs.Mass2);
	}

	inline static bool CollisionDataSortByMassPredicateMinToMax(const Chaos::FCollidingDataExt& Lhs, const Chaos::FCollidingDataExt& Rhs)
	{
		return FMath::Max(Lhs.Mass1, Lhs.Mass2) < FMath::Max(Rhs.Mass1, Rhs.Mass2);
	}

	inline static bool CollisionDataRandomShuffleSortPredicate(const Chaos::FCollidingDataExt& Lhs, const Chaos::FCollidingDataExt& Rhs)
	{
		return FMath::FRand() < 0.5f;
	}

	inline static bool BreakingDataSortByMassPredicateMaxToMin(const Chaos::FBreakingDataExt& Lhs, const Chaos::FBreakingDataExt& Rhs)
	{
		return Lhs.Mass > Rhs.Mass;
	}

	inline static bool BreakingDataSortByMassPredicateMinToMax(const Chaos::FBreakingDataExt& Lhs, const Chaos::FBreakingDataExt& Rhs)
	{
		return Lhs.Mass < Rhs.Mass;
	}

	inline static bool BreakingDataRandomShuffleSortPredicate(const Chaos::FBreakingDataExt& Lhs, const Chaos::FBreakingDataExt& Rhs)
	{
		return FMath::FRand() < 0.5f;
	}

	inline static bool TrailingDataSortByMassPredicateMaxToMin(const Chaos::FTrailingDataExt& Lhs, const Chaos::FTrailingDataExt& Rhs)
	{
		return Lhs.Mass > Rhs.Mass;
	}

	inline static bool TrailingDataSortByMassPredicateMinToMax(const Chaos::FTrailingDataExt& Lhs, const Chaos::FTrailingDataExt& Rhs)
	{
		return Lhs.Mass < Rhs.Mass;
	}

	inline static bool TrailingDataRandomShuffleSortPredicate(const Chaos::FTrailingDataExt& Lhs, const Chaos::FTrailingDataExt& Rhs)
	{
		return FMath::FRand() < 0.5f;
	}

	inline int32 GetLastSpawnedPointID()
	{
		return LastSpawnedPointID;
	}
	inline float GetSolverTime()
	{
		return SolverTime;
	}

	UE_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;
protected:

#if WITH_EDITORONLY_DATA
	UE_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

	UE_API void RegisterWithSolverEventManager(Chaos::FPhysicsSolver* Solver);
	UE_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

	UE_API void ResetInstData(FNDIChaosDestruction_InstanceData* InstData);

	UE_API void HandleCollisionEvents(const Chaos::FCollisionEventData& Event);
	UE_API void FilterAllCollisions(TArray<Chaos::FCollidingDataExt>& AllCollisionsArray);

	UE_API void SortCollisions(TArray<Chaos::FCollidingDataExt>& CollisionsArray);

	UE_API void GetCollisionsToSpawnFromCollisions(TArray<Chaos::FCollidingDataExt>& AllCollisionsArray,
							 			    TArray<Chaos::FCollidingDataExt>& CollisionsToSpawnArray);

	UE_API int32 SpawnParticlesFromCollision(FSolverData SolverData,
									  Chaos::FCollidingDataExt& Collision,
									  FNDIChaosDestruction_InstanceData* InstData,
									  float TimeData_MapsCreated,
									  int32 IdxSolver);
	
	UE_API void HandleBreakingEvents(const Chaos::FBreakingEventData& Event);
	UE_API void FilterAllBreakings(TArray<Chaos::FBreakingDataExt>& AllBreakingsArray);

	UE_API void SortBreakings(TArray<Chaos::FBreakingDataExt>& BreakingsArray);

	UE_API void GetBreakingsToSpawnFromBreakings(TArray<Chaos::FBreakingDataExt>& AllBreakingsArray,
										  TArray<Chaos::FBreakingDataExt>& BreakingsToSpawnArray);

	UE_API int32 SpawnParticlesFromBreaking(FSolverData SolverData,
									 Chaos::FBreakingDataExt& Breaking,
									 FNDIChaosDestruction_InstanceData* InstData,
									 float TimeData_MapsCreated,
									 int32 IdxSolver);
	   	  
	UE_API void HandleTrailingEvents(const Chaos::FTrailingEventData& Event);
	UE_API void FilterAllTrailings(TArray<Chaos::FTrailingDataExt>& AllTrailingsArray);

	UE_API void SortTrailings(TArray<Chaos::FTrailingDataExt>& TrailingsArray);

	UE_API void GetTrailingsToSpawnFromTrailings(TArray<Chaos::FTrailingDataExt>& AllTrailingsArray,
										  TArray<Chaos::FTrailingDataExt>& TrailingsToSpawnArray);

	UE_API int32 SpawnParticlesFromTrailing(FSolverData SolverData,
									 Chaos::FTrailingDataExt& Trailing,
									 FNDIChaosDestruction_InstanceData* InstData,
									 float TimeData_MapsCreated,
									 int32 IdxSolver);
	   	  
	UE_API bool CollisionCallback(FNDIChaosDestruction_InstanceData* InstData);
	UE_API bool BreakingCallback(FNDIChaosDestruction_InstanceData* InstData);
	UE_API bool TrailingCallback(FNDIChaosDestruction_InstanceData* InstData);

	UE_API virtual void PushToRenderThreadImpl() override;

	UPROPERTY()
	int32 LastSpawnedPointID;

	UPROPERTY()
	float LastSpawnTime;

	// Colors for debugging particles
	TArray<FVector> ColorArray;

	UPROPERTY()
	float SolverTime;

	UPROPERTY()
	float TimeStampOfLastProcessedData;

	bool ShouldSpawn;

	TArray<FSolverData> Solvers;
	TArray<Chaos::FCollidingDataExt> CollisionEvents;
	TArray<Chaos::FBreakingDataExt> BreakingEvents;
	TArray<Chaos::FTrailingDataExt> TrailingEvents;

	// geometry collection to track for trailing 
	TSet<TWeakObjectPtr<UGeometryCollectionComponent>> TrailingGeometryCollectionComponents;
};

struct FNiagaraDataInterfaceProxyChaosDestruction : public FNiagaraDataInterfaceProxy
{
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override;
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FNiagaraDIChaosDestruction_InstanceDataToPassToRT); }

	virtual void PostSimulate(const FNDIGpuComputePostSimulateContext& Context) override;

	void CreatePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	void DestroyInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	float SolverTime;
	int32 LastSpawnedPointID;

	TMap<FNiagaraSystemInstanceID, FNiagaraDIChaosDestruction_GPUData> SystemsToGPUInstanceData;
};

#undef UE_API
