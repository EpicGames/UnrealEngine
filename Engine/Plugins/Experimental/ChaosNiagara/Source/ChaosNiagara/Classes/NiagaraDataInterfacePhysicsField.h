// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"

#include "NiagaraDataInterfacePhysicsField.generated.h"

#define UE_API CHAOSNIAGARA_API

class FFieldSystemCommand;
struct FNiagaraDataInterfaceGeneratedFunction;

/** Data stored per physics asset instance on the render thread */
struct FNDIFieldRenderData
{
	/** Field render resource for gpu */
	class FPhysicsFieldResource* FieldResource;
	
	/** Time in seconds*/
	float TimeSeconds = 0.0;
};

/** Data stored per physics asset instance on the game thread */
struct FNDIPhysicsFieldData
{
	/** Initialize the resource */
	void Init(FNiagaraSystemInstance* SystemInstance);

	/** Update the commands */
	void Update(FNiagaraSystemInstance* SystemInstance);

	/** Release the buffers */
	void Release();

	/** Field render resource for gpu */
	class FPhysicsFieldResource* FieldResource;

	/** Field system commands for cpu */
	TArray<FFieldSystemCommand>	FieldCommands;

	/** Time in seconds*/
	float TimeSeconds = 0.0;
	
	FNiagaraLWCConverter LWCConverter;
};

/** Data Interface for the strand base */
UCLASS(MinimalAPI, EditInlineNew, Category = "Chaos", meta = (DisplayName = "Physics Field"))
class UNiagaraDataInterfacePhysicsField : public UNiagaraDataInterface
{
	GENERATED_BODY()

public:
	UE_API UNiagaraDataInterfacePhysicsField();

	/** UObject Interface */
	UE_API virtual void PostInitProperties() override;

	/** UNiagaraDataInterface Interface */
	UE_API virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
#if WITH_EDITOR
	virtual bool ShouldCompile(EShaderPlatform ShaderPlatform) const override { return !IsMobilePlatform(ShaderPlatform) && 
		IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM5);}
#endif
	UE_API virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	UE_API virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)override;
	UE_API virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FNDIPhysicsFieldData); }
	UE_API virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool HasPostSimulateTick() const override { return IsUsedWithCPUScript(); }

	/** GPU simulation functionality */
#if WITH_EDITORONLY_DATA
	UE_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	UE_API virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	UE_API virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	UE_API virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	UE_API virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	UE_API virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	UE_API virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

	/** Sample the vector field */
	UE_API void SamplePhysicsVectorField(FVectorVMExternalFunctionContext& Context);

	/** Sample the scalar field */
	UE_API void SamplePhysicsScalarField(FVectorVMExternalFunctionContext& Context);

	/** Sample the integer field */
	UE_API void SamplePhysicsIntegerField(FVectorVMExternalFunctionContext& Context);

	/** Get the field resolution */
	UE_API void GetPhysicsFieldResolution(FVectorVMExternalFunctionContext& Context);

	/** Get the field bounds */
	UE_API void GetPhysicsFieldBounds(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	UE_API virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

	/** Copy one niagara DI to this */
	UE_API virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;
};

/** Proxy to send data to gpu */
struct FNDIPhysicsFieldProxy : public FNiagaraDataInterfaceProxy
{
	/** Get the size of the data that will be passed to render*/
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FNDIFieldRenderData); }

	/** Get the data that will be passed to render*/
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override;

	/** Initialize the per instance data */
	void InitializePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	/** Destroy the proxy data if necessary */
	void DestroyPerInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	/** List of proxy data for each system instances*/
	TMap<FNiagaraSystemInstanceID, FNDIFieldRenderData> SystemInstancesToProxyData;
};

#undef UE_API
