// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosNiagara.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"


#define LOCTEXT_NAMESPACE "FChaosNiagaraModule"

void FChaosNiagaraModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("ChaosNiagara"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/Experimental/ChaosNiagara"), PluginShaderDir);
}

void FChaosNiagaraModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FChaosNiagaraModule, ChaosNiagara)
