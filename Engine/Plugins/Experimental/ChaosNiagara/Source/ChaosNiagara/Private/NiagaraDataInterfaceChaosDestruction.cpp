// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceChaosDestruction.h"
#include "Chaos/ChaosSolverActor.h"
#include "Engine/World.h"
#include "EventManager.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "EventsData.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "GeometryCollection/TransformCollection.h"
#include "NiagaraShaderParametersBuilder.h"
#include "GeometryCollectionProxyData.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraCompileHashVisitor.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#include "GeometryCollection/GeometryCollectionComponent.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "RenderGraphBuilder.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceChaosDestruction)

#define LOCTEXT_NAMESPACE "ChaosNiagaraDestructionDataInterface"
//#pragma optimize("", off)

DECLARE_STATS_GROUP(TEXT("ChaosNiagara"), STATGROUP_ChaosNiagara, STATCAT_Niagara);
DECLARE_CYCLE_STAT(TEXT("CollisionCallback"), STAT_CollisionCallback, STATGROUP_ChaosNiagara);
DECLARE_CYCLE_STAT(TEXT("TrailingCallback"), STAT_NiagaraTrailingCallback, STATGROUP_ChaosNiagara);
DECLARE_CYCLE_STAT(TEXT("BreakingCallback"), STAT_NiagaraBreakingCallback, STATGROUP_ChaosNiagara);
DECLARE_CYCLE_STAT(TEXT("CollisionCallbackSorting"), STAT_CollisionCallbackSorting, STATGROUP_ChaosNiagara);
DECLARE_CYCLE_STAT(TEXT("BreakingCallbackSorting"), STAT_BreakingCallbackSorting, STATGROUP_ChaosNiagara);
DECLARE_CYCLE_STAT(TEXT("TrailingCallbackSorting"), STAT_TrailingCallbackSorting, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumAllCollisions"), STAT_NiagaraNumAllCollisions, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumFilteredAllCollisions"), STAT_NiagaraNumFilteredAllCollisions, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumCollisionsToSpawnParticles"), STAT_NiagaraNumCollisionsToSpawnParticles, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumAllTrailings"), STAT_NiagaraNumAllTrailings, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumFilteredAllTrailings"), STAT_NiagaraNumFilteredAllTrailings, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumTrailingsToSpawnParticles"), STAT_NiagaraNumTrailingsToSpawnParticles, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumAllBreakings"), STAT_NiagaraNumAllBreakings, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumFilteredAllBreakings"), STAT_NiagaraNumFilteredAllBreakings, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumBreakingsToSpawnParticles"), STAT_NiagaraNumBreakingsToSpawnParticles, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumParticlesSpawnedFromCollisions"), STAT_NiagaraNumParticlesSpawnedFromCollisions, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumParticlesSpawnedFromTrailings"), STAT_NiagaraNumParticlesSpawnedFromTrailings, STATGROUP_ChaosNiagara);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumParticlesSpawnedFromBreaking"), STAT_NiagaraNumParticlesSpawnedFromBreakings, STATGROUP_ChaosNiagara);
DECLARE_MEMORY_STAT(TEXT("PhysicsProxyReverseMapping"), STAT_PhysicsProxyReverseMappingMemory, STATGROUP_ChaosNiagara);
DECLARE_MEMORY_STAT(TEXT("ParticleIndexReverseMapping"), STAT_ParticleIndexReverseMappingMemory, STATGROUP_ChaosNiagara);
DECLARE_MEMORY_STAT(TEXT("AllCollisionsData"), STAT_AllCollisionsDataMemory, STATGROUP_ChaosNiagara);
DECLARE_MEMORY_STAT(TEXT("AllCollisionsIndicesByPhysicsProxy"), STAT_AllCollisionsIndicesByPhysicsProxyMemory, STATGROUP_ChaosNiagara);
DECLARE_MEMORY_STAT(TEXT("AllBreakingsData"), STAT_AllBreakingsDataMemory, STATGROUP_ChaosNiagara);
DECLARE_MEMORY_STAT(TEXT("AllBreakingsIndicesByPhysicsProxy"), STAT_AllBreakingsIndicesByPhysicsProxyMemory, STATGROUP_ChaosNiagara);
DECLARE_MEMORY_STAT(TEXT("AllTrailingsData"), STAT_AllTrailingsDataMemory, STATGROUP_ChaosNiagara);
DECLARE_MEMORY_STAT(TEXT("AllTrailingsIndicesByPhysicsProxy"), STAT_AllTrailingsIndicesByPhysicsProxyMemory, STATGROUP_ChaosNiagara);

namespace NDIChaosDestructionLocal
{
	static const TCHAR* TemplateShaderFilePath = TEXT("/Plugin/Experimental/ChaosNiagara/NiagaraDataInterfaceChaosDestruction.ush");

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>,	PositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>,	VelocityBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>,	ExtentMinBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>,	ExtentMaxBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>,	VolumeBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	SolverIDBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>,	DensityBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>,	FrictionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>,	RestitutionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>,	SurfaceTypeBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>,	ColorBuffer)
		
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>,	IncomingLocationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>, IncomingAccumulatedImpulseBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>, IncomingNormalBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>, IncomingVelocity1Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>, IncomingVelocity2Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>, IncomingAngularVelocity1Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>, IncomingAngularVelocity2Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, IncomingMass1Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, IncomingMass2Buffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, IncomingTimeBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>, TransformTranslationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, TransformRotationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>, TransformScaleBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float3>, BoundsBuffer)

		SHADER_PARAMETER(int,	LastSpawnedPointID)
		SHADER_PARAMETER(float,	SolverTime)
	END_SHADER_PARAMETER_STRUCT()

	// Name of all the functions available in the data interface
	static const FName GetPositionName("GetPosition");
	static const FName GetNormalName("GetNormal");
	static const FName GetVelocityName("GetVelocity");
	static const FName GetAngularVelocityName("GetAngularVelocity");
	static const FName GetExtentMinName("GetExtentMin");
	static const FName GetExtentMaxName("GetExtentMax");
	static const FName GetVolumeName("GetVolume");
	static const FName GetParticleIdsToSpawnAtTimeName("GetParticleIdsToSpawnAtTime");
	static const FName GetPointTypeName("GetPointType");
	static const FName GetColorName("GetColor");
	static const FName GetSolverTimeName("GetSolverTime");
	static const FName GetDensityName("GetDensity");
	static const FName GetFrictionName("GetFriction");
	static const FName GetRestitutionName("GetRestitution");
	static const FName GetSurfaceTypeName("GetSurfaceType");
	static const FName GetTransformName("GetTransform");
	static const FName GetSizeName("GetSize");
	static const FName GetCollisionDataName("GetCollisionData");
	static const FName GetBreakingDataName("GetBreakingData");
	static const FName GetTrailingDataName("GetTrailingData");

	TArray<FVector3f> CopyVectorArray(TArrayView<FVector> DoubleArray)
	{
		TArray<FVector3f> FloatArray;
		FloatArray.AddUninitialized(DoubleArray.Num());
		for ( int i=0; i < DoubleArray.Num(); ++i )
		{
			FloatArray[i] = FVector3f(DoubleArray[i]);	//LWC_TODO: Should we convert the space?
		}
		return FloatArray;
	}

	TArray<FQuat4f> CopyQuatArray(TArrayView<FQuat> DoubleArray)
	{
		TArray<FQuat4f> FloatArray;
		FloatArray.AddUninitialized(DoubleArray.Num());
		for (int i = 0; i < DoubleArray.Num(); ++i)
		{
			FloatArray[i] = FQuat4f(DoubleArray[i]);	//LWC_TODO: Should we convert the space?
		}
		return FloatArray;
	}

	template<typename T>
	void QueueBufferUpload(const FNiagaraDataInterfaceSetShaderParametersContext& Context, FRDGBufferSRVRef& ParameterBinding, FRDGBufferSRVRef& ExistingSRV, TConstArrayView<T> ArrayData, const T& DefaultValue, const EPixelFormat PixelFormat, const TCHAR* BufferName)
	{
		// Buffer Used?
		if (Context.IsResourceBound(&ParameterBinding) == false)
		{
			return;
		}

		// Buffer needs uploading this frame?
		if (ExistingSRV == nullptr)
		{
			if (ArrayData.Num() == 0)
			{
				ArrayData = MakeArrayView<const T>(&DefaultValue, 1);
			}

			check(GPixelFormats[PixelFormat].BlockBytes == sizeof(T));

			FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
			FRDGBufferRef CreatedBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(T), ArrayData.Num()), BufferName);
			GraphBuilder.QueueBufferUpload(CreatedBuffer, ArrayData.GetData(), ArrayData.GetTypeSize() * ArrayData.Num());

			ExistingSRV = GraphBuilder.CreateSRV(CreatedBuffer, PixelFormat);
		}

		// Done assign the result
		ParameterBinding = ExistingSRV;
	}

	template<>
	void QueueBufferUpload<FVector3f>(const FNiagaraDataInterfaceSetShaderParametersContext& Context, FRDGBufferSRVRef& ParameterBinding, FRDGBufferSRVRef& ExistingSRV, TConstArrayView<FVector3f> ArrayData, const FVector3f& DefaultValue, const EPixelFormat PixelFormat, const TCHAR* BufferName)
	{
		// Buffer Used?
		if (Context.IsResourceBound(&ParameterBinding) == false)
		{
			return;
		}

		// Buffer needs uploading this frame?
		if (ExistingSRV == nullptr)
		{
			if (ArrayData.Num() == 0)
			{
				ArrayData = MakeArrayView<const FVector3f>(&DefaultValue, 1);
			}

			check(PixelFormat == PF_A32B32G32R32F);

			FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
			FRDGBufferRef CreatedBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FVector4f), ArrayData.Num()), BufferName);
			FVector4f* BufferData = GraphBuilder.AllocPODArray<FVector4f>(ArrayData.Num());
			for (int i = 0; i < ArrayData.Num(); ++i)
			{
				BufferData[i] = FVector4f(ArrayData[i]);
			}
			GraphBuilder.QueueBufferUpload(CreatedBuffer, BufferData, sizeof(FVector4f) * ArrayData.Num(), ERDGInitialDataFlags::NoCopy);
			ExistingSRV = GraphBuilder.CreateSRV(CreatedBuffer, PixelFormat);
		}

		// Done assign the result
		ParameterBinding = ExistingSRV;
	}

	void GetEmptyBuffer(const FNiagaraDataInterfaceSetShaderParametersContext& Context, FRDGBufferSRVRef& ParameterBinding, const EPixelFormat PixelFormat)
	{
		if (Context.IsResourceBound(&ParameterBinding) == true)
		{
			ParameterBinding = Context.GetComputeDispatchInterface().GetEmptyBufferSRV(Context.GetGraphBuilder(), PixelFormat);
		}
	}
}

UNiagaraDataInterfaceChaosDestruction::UNiagaraDataInterfaceChaosDestruction(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, DataSourceType(EDataSourceTypeEnum::ChaosNiagara_DataSourceType_Collision)
	, DataProcessFrequency(10)
	, MaxNumberOfDataEntriesToSpawn(50)
	, DoSpawn(true)
	, SpawnMultiplierMinMax(FVector2D(1, 1))
	, SpawnChance(1.f)
	, ImpulseToSpawnMinMax(FVector2D(-1.f, -1.f))
	, SpeedToSpawnMinMax(FVector2D(-1.f, -1.f))
	, MassToSpawnMinMax(FVector2D(-1.f, -1.f))
	, ExtentMinToSpawnMinMax(FVector2D(-1.f, -1.f))
	, ExtentMaxToSpawnMinMax(FVector2D(-1.f, -1.f))
	, VolumeToSpawnMinMax(FVector2D(-1.f, -1.f))
	, SolverTimeToSpawnMinMax(FVector2D(-1.f, -1.f))
	, SurfaceTypeToSpawn(-1.f)
	, LocationFilteringMode(ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive)
	, LocationXToSpawn(ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_None)
	, LocationXToSpawnMinMax(FVector2D(0.f, 0.f))
	, LocationYToSpawn(ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_None)
	, LocationYToSpawnMinMax(FVector2D(0.f, 0.f))
	, LocationZToSpawn(ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_None)
	, LocationZToSpawnMinMax(FVector2D(0.f, 0.f))
	, TrailMinSpeedToSpawn(50.f)
	, DataSortingType(EDataSortTypeEnum::ChaosNiagara_DataSortType_NoSorting)
	, bGetExternalCollisionData(false)
	, DoSpatialHash(false)
	, SpatialHashVolumeMin(FVector(-100.f))
	, SpatialHashVolumeMax(FVector(100.f))
	, SpatialHashVolumeCellSize(FVector(10.f))
	, MaxDataPerCell(1)
	, bApplyMaterialsFilter(false)
	, bGetExternalBreakingData(true)
	, bGetExternalTrailingData(false)
	, RandomPositionMagnitudeMinMax(FVector2D(0.f, 0.f))
	, InheritedVelocityMultiplier(1.f)
	, RandomVelocityGenerationType(ERandomVelocityGenerationTypeEnum::ChaosNiagara_RandomVelocityGenerationType_RandomDistribution)
	, RandomVelocityMagnitudeMinMax(FVector2D(1.f, 2.f))
	, SpreadAngleMax(30.f)
	, VelocityOffsetMin(FVector(ForceInitToZero))
	, VelocityOffsetMax(FVector(ForceInitToZero))
	, FinalVelocityMagnitudeMinMax(FVector2D(-1.f, -1.f))
	, MaxLatency(1.f)
	, DebugType(EDebugTypeEnum::ChaosNiagara_DebugType_NoDebug)
	, LastSpawnedPointID(-1)
	, LastSpawnTime(-1.f)
	, SolverTime(0.f)
	, TimeStampOfLastProcessedData(-1.f)
	, ShouldSpawn(true)

{
	// Colors to visualize particles for debugging
	ColorArray.Add({ 1.0, 1.0, 1.0 }); // White
	ColorArray.Add({ 1.0, 0.0, 0.0 }); // Red
	ColorArray.Add({ 0.0, 1.0, 0.0 }); // Lime
	ColorArray.Add({ 0.0, 0.0, 1.0 }); // Blue
	ColorArray.Add({ 1.0, 1.0, 0.0 }); // Yellow
	ColorArray.Add({ 0.0, 1.0, 1.0 }); // Cyan
	ColorArray.Add({ 1.0, 0.0, 1.0 }); // Magenta
	ColorArray.Add({ 0.75, 0.75, 0.75 }); // Silver
	ColorArray.Add({ 0.5, 0.5, 0.5 }); // Gray
	ColorArray.Add({ 0.5, 0.0, 0.0 }); // Maroon
	ColorArray.Add({ 0.5, 0.5, 0.0 }); // Olive
	ColorArray.Add({ 0.0, 0.5, 0.0 }); // Green
	ColorArray.Add({ 0.5, 0.0, 0.5 }); // Purple
	ColorArray.Add({ 0.0, 0.5, 0.5 }); // Teal
	ColorArray.Add({ 0.0, 0.0, 0.5 }); // Navy
	ColorArray.Add({ 1.0, 165.0 / 255.0, 0.5 }); // Orange
	ColorArray.Add({ 1.0, 215.0 / 255.0, 0.5 }); // Gold
	ColorArray.Add({ 154.0 / 255.0, 205.0 / 255.0, 50.0 / 255.0 }); // Yellow green
	ColorArray.Add({ 127.0 / 255.0, 255.0 / 255.0, 212.0 / 255.0 }); // Aqua marine

	Solvers.Reset();

	Proxy.Reset(new FNiagaraDataInterfaceProxyChaosDestruction());
	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceChaosDestruction::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags DIFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), DIFlags);

		ENiagaraTypeRegistryFlags EventFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter |
			ENiagaraTypeRegistryFlags::AllowPayload;
		FNiagaraTypeRegistry::Register(FChaosDestructionEvent::StaticStruct(), EventFlags);
	}

	LastSpawnedPointID = -1;
	LastSpawnTime = -1.f;
	TimeStampOfLastProcessedData = -1.f;
	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceChaosDestruction::PostLoad()
{
	Super::PostLoad();

	LastSpawnedPointID = -1;
	LastSpawnTime = -1.f;
	TimeStampOfLastProcessedData = -1.f;

	MarkRenderDataDirty();
}

void UNiagaraDataInterfaceChaosDestruction::BeginDestroy()
{
	Super::BeginDestroy();

	for (FSolverData& SolverData : Solvers)
	{
		Chaos::FEventManager* EventManager = SolverData.Solver->GetEventManager();
		if (EventManager)
		{
			EventManager->UnregisterHandler(Chaos::EEventType::Collision, this);
			EventManager->UnregisterHandler(Chaos::EEventType::Breaking, this);
			EventManager->UnregisterHandler(Chaos::EEventType::Trailing, this);
		}
	}

	Solvers.Reset();
}

#if WITH_EDITOR

void UNiagaraDataInterfaceChaosDestruction::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, ChaosSolverActorSet))
		{
			Modify();
			if (ChaosSolverActorSet.Num())
			{
				LastSpawnedPointID = -1;
				LastSpawnTime = -1.f;
				TimeStampOfLastProcessedData = -1.f;
			}
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, ChaosBreakingMaterialSet))
		{
			Modify();
			if (ChaosBreakingMaterialSet.Num())
			{
				/**/
			}
		}
		// Validate inputs
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, DataProcessFrequency))
		{
			DataProcessFrequency = FMath::Max(1, DataProcessFrequency);
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, MaxNumberOfDataEntriesToSpawn))
		{
			MaxNumberOfDataEntriesToSpawn = FMath::Max(0, MaxNumberOfDataEntriesToSpawn);
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, SpawnMultiplierMinMax))
		{
			if (PropertyChangedEvent.Property->GetFName() == FName("X"))
			{
				SpawnMultiplierMinMax.X = FMath::Max(0.f, SpawnMultiplierMinMax.X);
			}
			else if (PropertyChangedEvent.Property->GetFName() == FName("Y"))
			{
				SpawnMultiplierMinMax.Y = FMath::Max(0.f, SpawnMultiplierMinMax.Y);
			}
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, SpawnChance))
		{
			SpawnChance = FMath::Clamp(SpawnChance, 0.f, 1.f);
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, SpatialHashVolumeCellSize))
		{
			SpatialHashVolumeCellSize.X = FMath::Max<FVector::FReal>(1.0, SpatialHashVolumeCellSize.X);
			SpatialHashVolumeCellSize.Y = FMath::Max<FVector::FReal>(1.0, SpatialHashVolumeCellSize.Y);
			SpatialHashVolumeCellSize.Z = FMath::Max<FVector::FReal>(1.0, SpatialHashVolumeCellSize.Z);
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, MaxDataPerCell))
		{
			MaxDataPerCell = FMath::Max(0, MaxDataPerCell);
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, RandomVelocityMagnitudeMinMax))
		{
			if (PropertyChangedEvent.Property->GetFName() == FName("X"))
			{
				RandomVelocityMagnitudeMinMax.X = FMath::Max(0.f, RandomVelocityMagnitudeMinMax.X);
			}
			else if (PropertyChangedEvent.Property->GetFName() == FName("Y"))
			{
				RandomVelocityMagnitudeMinMax.Y = FMath::Max(0.f, RandomVelocityMagnitudeMinMax.Y);
			}
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, SpreadAngleMax))
		{
			SpreadAngleMax = FMath::Clamp(SpreadAngleMax, 0.f, 90.f);
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceChaosDestruction, MaxLatency))
		{
			MaxLatency = FMath::Max(0.f, MaxLatency);
		}
	}

	MarkRenderDataDirty();
}

#endif

bool UNiagaraDataInterfaceChaosDestruction::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	if (UNiagaraDataInterfaceChaosDestruction* DestinationChaosDestruction = CastChecked<UNiagaraDataInterfaceChaosDestruction>(Destination))
	{
		DestinationChaosDestruction->ChaosSolverActorSet = ChaosSolverActorSet;
		DestinationChaosDestruction->DataSourceType = DataSourceType;
		DestinationChaosDestruction->DataProcessFrequency = DataProcessFrequency;
		DestinationChaosDestruction->MaxNumberOfDataEntriesToSpawn = MaxNumberOfDataEntriesToSpawn;
		DestinationChaosDestruction->DoSpawn = DoSpawn;
		DestinationChaosDestruction->ShouldSpawn = ShouldSpawn;
		DestinationChaosDestruction->SpawnMultiplierMinMax = SpawnMultiplierMinMax;
		DestinationChaosDestruction->SpawnChance = SpawnChance;
		DestinationChaosDestruction->ImpulseToSpawnMinMax = ImpulseToSpawnMinMax;
		DestinationChaosDestruction->SpeedToSpawnMinMax = SpeedToSpawnMinMax;
		DestinationChaosDestruction->MassToSpawnMinMax = MassToSpawnMinMax;
		DestinationChaosDestruction->ExtentMinToSpawnMinMax = ExtentMinToSpawnMinMax;
		DestinationChaosDestruction->ExtentMaxToSpawnMinMax = ExtentMaxToSpawnMinMax;
		DestinationChaosDestruction->VolumeToSpawnMinMax = VolumeToSpawnMinMax;
		DestinationChaosDestruction->SolverTimeToSpawnMinMax = SolverTimeToSpawnMinMax;
		DestinationChaosDestruction->SurfaceTypeToSpawn = SurfaceTypeToSpawn;
		DestinationChaosDestruction->LocationFilteringMode = LocationFilteringMode;
		DestinationChaosDestruction->LocationXToSpawn = LocationXToSpawn;
		DestinationChaosDestruction->LocationXToSpawnMinMax = LocationXToSpawnMinMax;
		DestinationChaosDestruction->LocationYToSpawn = LocationYToSpawn;
		DestinationChaosDestruction->LocationYToSpawnMinMax = LocationYToSpawnMinMax;
		DestinationChaosDestruction->LocationZToSpawn = LocationZToSpawn;
		DestinationChaosDestruction->LocationZToSpawnMinMax = LocationZToSpawnMinMax;
		DestinationChaosDestruction->DataSortingType = DataSortingType;
		DestinationChaosDestruction->DoSpatialHash = DoSpatialHash;
		DestinationChaosDestruction->bGetExternalCollisionData = bGetExternalCollisionData;
		DestinationChaosDestruction->bGetExternalBreakingData = bGetExternalBreakingData;
		DestinationChaosDestruction->bGetExternalTrailingData = bGetExternalTrailingData;
		DestinationChaosDestruction->SpatialHashVolumeMin = SpatialHashVolumeMin;
		DestinationChaosDestruction->SpatialHashVolumeMax = SpatialHashVolumeMax;
		DestinationChaosDestruction->SpatialHashVolumeCellSize = SpatialHashVolumeCellSize;
		DestinationChaosDestruction->MaxDataPerCell = MaxDataPerCell;
		DestinationChaosDestruction->bApplyMaterialsFilter = bApplyMaterialsFilter;
		DestinationChaosDestruction->ChaosBreakingMaterialSet = ChaosBreakingMaterialSet;
		DestinationChaosDestruction->RandomPositionMagnitudeMinMax = RandomPositionMagnitudeMinMax;
		DestinationChaosDestruction->InheritedVelocityMultiplier = InheritedVelocityMultiplier;
		DestinationChaosDestruction->RandomVelocityGenerationType = RandomVelocityGenerationType;
		DestinationChaosDestruction->RandomVelocityMagnitudeMinMax = RandomVelocityMagnitudeMinMax;
		DestinationChaosDestruction->SpreadAngleMax = SpreadAngleMax;
		DestinationChaosDestruction->VelocityOffsetMin = VelocityOffsetMin;
		DestinationChaosDestruction->VelocityOffsetMax = VelocityOffsetMax;
		DestinationChaosDestruction->FinalVelocityMagnitudeMinMax = FinalVelocityMagnitudeMinMax;
		DestinationChaosDestruction->MaxLatency = MaxLatency;
		DestinationChaosDestruction->DebugType = DebugType;
		DestinationChaosDestruction->LastSpawnedPointID = LastSpawnedPointID;
		DestinationChaosDestruction->LastSpawnTime = LastSpawnTime;
		DestinationChaosDestruction->TimeStampOfLastProcessedData = TimeStampOfLastProcessedData;
		DestinationChaosDestruction->SolverTime = SolverTime;
		DestinationChaosDestruction->MarkRenderDataDirty();

		return true;
	}

	return false;
}

bool UNiagaraDataInterfaceChaosDestruction::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}

	const UNiagaraDataInterfaceChaosDestruction* OtherChaosDestruction = Cast<const UNiagaraDataInterfaceChaosDestruction>(Other);
	if (OtherChaosDestruction == nullptr)
	{
		return false;
	}

	if (OtherChaosDestruction->ChaosSolverActorSet.Num() != ChaosSolverActorSet.Num())
	{
		return false;
	}


	TArray<AChaosSolverActor*> SolverActors = ChaosSolverActorSet.Array();
	TArray<AChaosSolverActor*> OtherSolverActors = OtherChaosDestruction->ChaosSolverActorSet.Array();

	bool bResult = true;
	for (int32 Idx = 0; Idx < SolverActors.Num(); ++Idx)
	{
		AChaosSolverActor* ThisActor = SolverActors[Idx];
		AChaosSolverActor* OtherActor = OtherSolverActors[Idx];

		if(ThisActor && OtherActor)
		{
			bResult = bResult && ThisActor->GetName().Equals(OtherActor->GetName());
		}
	}

	return bResult
		&& OtherChaosDestruction->DoSpawn == DoSpawn
		&& OtherChaosDestruction->ShouldSpawn == ShouldSpawn
		&& OtherChaosDestruction->DataSourceType == DataSourceType
		&& OtherChaosDestruction->DataProcessFrequency == DataProcessFrequency
		&& OtherChaosDestruction->MaxNumberOfDataEntriesToSpawn == MaxNumberOfDataEntriesToSpawn
		&& OtherChaosDestruction->SpawnMultiplierMinMax == SpawnMultiplierMinMax
		&& OtherChaosDestruction->SpawnChance == SpawnChance
		&& OtherChaosDestruction->ImpulseToSpawnMinMax == ImpulseToSpawnMinMax
		&& OtherChaosDestruction->SpeedToSpawnMinMax == SpeedToSpawnMinMax
		&& OtherChaosDestruction->MassToSpawnMinMax == MassToSpawnMinMax
		&& OtherChaosDestruction->ExtentMinToSpawnMinMax == ExtentMinToSpawnMinMax
		&& OtherChaosDestruction->ExtentMaxToSpawnMinMax == ExtentMaxToSpawnMinMax
		&& OtherChaosDestruction->VolumeToSpawnMinMax == VolumeToSpawnMinMax
		&& OtherChaosDestruction->SolverTimeToSpawnMinMax == SolverTimeToSpawnMinMax
		&& OtherChaosDestruction->SurfaceTypeToSpawn == SurfaceTypeToSpawn
		&& OtherChaosDestruction->LocationFilteringMode == LocationFilteringMode
		&& OtherChaosDestruction->LocationXToSpawn == LocationXToSpawn
		&& OtherChaosDestruction->LocationXToSpawnMinMax == LocationXToSpawnMinMax
		&& OtherChaosDestruction->LocationYToSpawn == LocationYToSpawn
		&& OtherChaosDestruction->LocationYToSpawnMinMax == LocationYToSpawnMinMax
		&& OtherChaosDestruction->LocationZToSpawn == LocationZToSpawn
		&& OtherChaosDestruction->LocationZToSpawnMinMax == LocationZToSpawnMinMax
		&& OtherChaosDestruction->DataSortingType == DataSortingType
		&& OtherChaosDestruction->DoSpatialHash == DoSpatialHash
		&& OtherChaosDestruction->bGetExternalCollisionData == bGetExternalCollisionData
		&& OtherChaosDestruction->bGetExternalBreakingData == bGetExternalBreakingData
		&& OtherChaosDestruction->bGetExternalTrailingData == bGetExternalTrailingData
		&& OtherChaosDestruction->SpatialHashVolumeMin == SpatialHashVolumeMin
		&& OtherChaosDestruction->SpatialHashVolumeMax == SpatialHashVolumeMax
		&& OtherChaosDestruction->SpatialHashVolumeCellSize == SpatialHashVolumeCellSize
		&& OtherChaosDestruction->MaxDataPerCell == MaxDataPerCell
		&& OtherChaosDestruction->bApplyMaterialsFilter == bApplyMaterialsFilter
		//&& OtherChaosDestruction->ChaosBreakingMaterialSet == ChaosBreakingMaterialSet // Error	C2678	binary '==': no operator found which takes a left - hand operand of type 'const TSet<UPhysicalMaterial *,DefaultKeyFuncs<InElementType,false>,FDefaultSetAllocator>' (or there is no acceptable conversion)	ChaosOdin	Z : \Epic\Morten.Vassvik_DESKTOP - Dev - Destruction\Engine\Plugins\Experimental\ChaosNiagara\Source\ChaosNiagara\Private\NiagaraDataInterfaceChaosDestruction.cpp	369
		&& OtherChaosDestruction->RandomPositionMagnitudeMinMax == RandomPositionMagnitudeMinMax
		&& OtherChaosDestruction->InheritedVelocityMultiplier == InheritedVelocityMultiplier
		&& OtherChaosDestruction->RandomVelocityGenerationType == RandomVelocityGenerationType
		&& OtherChaosDestruction->RandomVelocityMagnitudeMinMax == RandomVelocityMagnitudeMinMax
		&& OtherChaosDestruction->SpreadAngleMax == SpreadAngleMax
		&& OtherChaosDestruction->VelocityOffsetMin == VelocityOffsetMin
		&& OtherChaosDestruction->VelocityOffsetMax == VelocityOffsetMax
		&& OtherChaosDestruction->FinalVelocityMagnitudeMinMax == FinalVelocityMagnitudeMinMax
		&& OtherChaosDestruction->MaxLatency == MaxLatency
		&& OtherChaosDestruction->DebugType == DebugType;
}

int32 UNiagaraDataInterfaceChaosDestruction::PerInstanceDataSize()const
{
	return sizeof(FNDIChaosDestruction_InstanceData);
}

void UNiagaraDataInterfaceChaosDestruction::RegisterWithSolverEventManager(Chaos::FPhysicsSolver* Solver)
{
	Chaos::FEventManager* EventManager = Solver->GetEventManager();
	if (EventManager)
	{
		switch (DataSourceType)
		{
		case EDataSourceTypeEnum::ChaosNiagara_DataSourceType_Collision:
			Solver->SetGenerateCollisionData(true);
			EventManager->RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, this, &UNiagaraDataInterfaceChaosDestruction::HandleCollisionEvents);
			break;
		case EDataSourceTypeEnum::ChaosNiagara_DataSourceType_Breaking:
			Solver->SetGenerateBreakingData(true);
			EventManager->RegisterHandler<Chaos::FBreakingEventData>(Chaos::EEventType::Breaking, this, &UNiagaraDataInterfaceChaosDestruction::HandleBreakingEvents);
			break;
		case EDataSourceTypeEnum::ChaosNiagara_DataSourceType_Trailing:
			Solver->SetGenerateBreakingData(true);
			EventManager->RegisterHandler<Chaos::FBreakingEventData>(Chaos::EEventType::Breaking, this, &UNiagaraDataInterfaceChaosDestruction::HandleBreakingEvents);
			Solver->SetGenerateTrailingData(true);
			EventManager->RegisterHandler<Chaos::FTrailingEventData>(Chaos::EEventType::Trailing, this, &UNiagaraDataInterfaceChaosDestruction::HandleTrailingEvents);
			break;
		}
	}
}


bool UNiagaraDataInterfaceChaosDestruction::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIChaosDestruction_InstanceData* InstData = new (PerInstanceData) FNDIChaosDestruction_InstanceData();

	LastSpawnedPointID = -1;
	LastSpawnTime = -1.0f;
	TimeStampOfLastProcessedData = -1.f;
	// No need to bind to non-game solvers as they will never generate events.
	UWorld* CurrentWorld = GetWorld();
	if(CurrentWorld && CurrentWorld->IsGameWorld())
	{
		// If there is no SolverActor specified need to grab the WorldSolver
		if(ChaosSolverActorSet.Num() == 0)
		{
			if(SystemInstance)
			{
				if(UWorld* World = SystemInstance->GetWorld())
				{
					int32 NewIdx = Solvers.Add(FSolverData());

					FSolverData& SolverData = Solvers[NewIdx];
					SolverData.PhysScene = World->GetPhysicsScene();
					SolverData.Solver = SolverData.PhysScene->GetSolver();

					RegisterWithSolverEventManager(SolverData.Solver);
				}
			}
		}
		//else
		{
			// #todo : are solver actors going to exist going forwards?
			for(AChaosSolverActor* SolverActor : ChaosSolverActorSet)
			{
				if(SolverActor)
				{
					if(Chaos::FPhysicsSolver* Solver = SolverActor->GetSolver())
					{
						int32 NewIdx = Solvers.Add(FSolverData());

						FSolverData& SolverData = Solvers[NewIdx];
						SolverData.PhysScene = SolverActor->GetPhysicsScene().Get();
						SolverData.Solver = Solver;

						RegisterWithSolverEventManager(Solver);
					}
				}
			}
		}
	}

	ResetInstData(InstData);

	if (SystemInstance)
	{
		FNiagaraDataInterfaceProxyChaosDestruction* ThisProxy = GetProxyAs<FNiagaraDataInterfaceProxyChaosDestruction>();
		ENQUEUE_RENDER_COMMAND(FNiagaraChaosDestructionDICreateRTInstance)(
			[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandList& CmdList)
		{
			ThisProxy->CreatePerInstanceData(InstanceID);
		}
		);
	}
	return true;
}

void UNiagaraDataInterfaceChaosDestruction::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIChaosDestruction_InstanceData* InstData = (FNDIChaosDestruction_InstanceData*)PerInstanceData;
	InstData->~FNDIChaosDestruction_InstanceData();

	check(SystemInstance);
	check(Proxy);
	if (SystemInstance)
	{
		FNiagaraDataInterfaceProxyChaosDestruction* ThisProxy = GetProxyAs<FNiagaraDataInterfaceProxyChaosDestruction>();
		ENQUEUE_RENDER_COMMAND(FNiagaraDIChaosDestructionDestroyInstanceData) (
			[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
			{
				ThisProxy->DestroyInstanceData(InstanceID);
			}
		);
	}
}

#if CHAOS_PARTICLEHANDLE_TODO
void GetMeshExtData(FSolverData SolverData,
					const int32 ParticleIndex,
					const TArray<PhysicsProxyWrapper>& PhysicsProxyReverseMapping,
					const TArray<int32>& ParticleIndexReverseMapping,
					float& BoundingboxVolume,
					float& BoundingboxExtentMin,
					float& BoundingboxExtentMax,
					FBox& BoundingBox,
					int32& SurfaceType,
					Chaos::TRigidTransform<float, 3>& Transform,
					UPhysicalMaterial*& PhysicalMaterial)
{
	PhysicalMaterial = nullptr;
	if (ParticleIndex < 0)
	{
		BoundingboxVolume = 1000000.f;
		BoundingboxExtentMin = 100.f;
		BoundingboxExtentMax = 100.f;
		SurfaceType = 0;
	}
	else if (PhysicsProxyReverseMapping[ParticleIndex].Type == EPhysicsProxyType::GeometryCollectionType)
	{
		// Since we are touching game objects below, I want to make sure that we're not off in some random thread.
		ensure(IsInGameThread());
		if (IPhysicsProxyBase* PhysicsProxy = PhysicsProxyReverseMapping[ParticleIndex].PhysicsProxy)
		{
			if (UGeometryCollectionComponent* GeometryCollectionComponent = SolverData.PhysScene->GetOwningComponent<UGeometryCollectionComponent>(PhysicsProxy))
			{
				if (const UGeometryCollection* GeometryCollection = GeometryCollectionComponent->GetRestCollection())
				{
					if (const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionObject = GeometryCollection->GetGeometryCollection())
					{
						//int NumTransforms = GeometryCollectionObject->NumElements(FGeometryCollection::TransformGroup);
						if (!ensure(0 <= ParticleIndex && ParticleIndex < ParticleIndexReverseMapping.Num()))
						{
							return;
						}
						int32 TransformIndex = ParticleIndexReverseMapping[ParticleIndex];
						//ensure(TransformIndex < NumTransforms);

						//int NumGeoms = GeometryCollectionObject->NumElements(FGeometryCollection::GeometryGroup);
						int32 GeometryGroupIndex = GeometryCollectionObject->TransformToGeometryIndex[TransformIndex];
						//ensure(0 <= GeometryGroupIndex && GeometryGroupIndex < NumGeoms);

						if (!ensure(0 <= GeometryGroupIndex && GeometryGroupIndex < GeometryCollectionObject->BoundingBox.Num()))
						{
							return;
						}
						BoundingBox = GeometryCollectionObject->BoundingBox[GeometryGroupIndex];

						FVector Extents = BoundingBox.GetSize();
						BoundingboxExtentMin = FMath::Min3(Extents[0], Extents[1], Extents[2]);
						BoundingboxExtentMax = FMath::Max3(Extents[0], Extents[1], Extents[2]);
						BoundingboxVolume = BoundingBox.GetVolume();

						// Get data from MareialID[]
						int32 FaceStartIndex = GeometryCollectionObject->FaceStart[GeometryGroupIndex];
						int32 MaterialID = GeometryCollectionObject->MaterialID[FaceStartIndex];

						UMaterialInterface* Material = GeometryCollectionComponent->GetMaterial(MaterialID);
						ensure(Material);
						if (Material)
						{
							PhysicalMaterial = Material->GetPhysicalMaterial();
							ensure(PhysicalMaterial);
							if (PhysicalMaterial)
							{
								SurfaceType = PhysicalMaterial->SurfaceType;
							}
						}
					}
				}
				if (const FGeometryCollectionPhysicsProxy* GeomCollectionPhysicsProxy = GeometryCollectionComponent->GetPhysicsProxy())
				{
					if (const FGeometryCollectionResults* PhysResult = GeomCollectionPhysicsProxy->GetConsumerResultsGT())
					{
						Transform = PhysResult->ParticleToWorldTransforms[ParticleIndex - PhysResult->BaseIndex];
					}
				}
			}
		}
	}
}

void GetMesPhysicalData(FSolverData SolverData,
						const int32 ParticleIndex,
						const TArray<PhysicsProxyWrapper>& PhysicsProxyReverseMapping,
						const TArray<int32>& ParticleIndexReverseMapping,
						FLinearColor& Color,
						float& Friction,
						float& Restitution,
						float& Density)
{
	static FMaterialParameterInfo ChaosDestructionColorName[3] = {
		FMaterialParameterInfo(FName("ChaosDestructionColor1")),
		FMaterialParameterInfo(FName("ChaosDestructionColor2")),
		FMaterialParameterInfo(FName("ChaosDestructionColor3")),
	};
	
	if (ParticleIndex < 0)
	{
		Friction = 0.7f;
		Restitution = 0.3f;
		Density = 1.0f;
	}
	else if (PhysicsProxyReverseMapping[ParticleIndex].Type == EPhysicsProxyType::GeometryCollectionType)
	{
		// Since we are touching game objects below, I want to make sure that we're not off in some random thread.
		ensure(IsInGameThread());

		if (IPhysicsProxyBase* PhysicsProxy = PhysicsProxyReverseMapping[ParticleIndex].PhysicsProxy)
		{
			if (UGeometryCollectionComponent* GeometryCollectionComponent = SolverData.PhysScene->GetOwningComponent<UGeometryCollectionComponent>(PhysicsProxy))
			{
				if (const UGeometryCollection* GeometryCollection = GeometryCollectionComponent->GetRestCollection())
				{
					if (const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionObject = GeometryCollection->GetGeometryCollection())
					{
						//int NumTransforms = GeometryCollectionObject->NumElements(FGeometryCollection::TransformGroup);
						if (!ensure(0 <= ParticleIndex && ParticleIndex < ParticleIndexReverseMapping.Num()))
						{
							return;
						}
						int32 TransformIndex = ParticleIndexReverseMapping[ParticleIndex];
						//ensure(TransformIndex < NumTransforms);

						//int NumGeoms = GeometryCollectionObject->NumElements(FGeometryCollection::GeometryGroup);
						if (!ensure(0 <= TransformIndex && TransformIndex < GeometryCollectionObject->TransformToGeometryIndex.Num()))
						{
							return;
						}
						int32 GeometryGroupIndex = GeometryCollectionObject->TransformToGeometryIndex[TransformIndex];
						//ensure(0 <= GeometryGroupIndex && GeometryGroupIndex < NumGeoms);
						
						
						if (!ensure(0 <= GeometryGroupIndex && GeometryGroupIndex < GeometryCollectionObject->BoundingBox.Num()))
						{
							return;
						}
						FBox BoundingBox = GeometryCollectionObject->BoundingBox[GeometryGroupIndex];

						// Get data from MaterialID[]
						int32 FaceStartIndex = GeometryCollectionObject->FaceStart[GeometryGroupIndex];
						int32 MaterialID = GeometryCollectionObject->MaterialID[FaceStartIndex];
						// For now let's use the first material
						MaterialID = 0;

						UMaterialInterface* Material = GeometryCollectionComponent->GetMaterial(MaterialID);
						ensure(Material);
						if (Material)
						{
							int RandVal = FMath::RandRange(0, sizeof(ChaosDestructionColorName) / sizeof(FMaterialParameterInfo) - 1);

							FLinearColor ChaosDestructionColor;
							if (Material->GetVectorParameterValue(ChaosDestructionColorName[RandVal], ChaosDestructionColor))
							{
								Color = ChaosDestructionColor;
							}

							UPhysicalMaterial* PhysicalMaterial = Material->GetPhysicalMaterial();
							ensure(PhysicalMaterial);
							if (PhysicalMaterial)

							{
								//UE_LOGF(LogScript, Warning, "GetMesPhysicalData: Name = %ls", *PhysicalMaterial->GetName());
								Friction = PhysicalMaterial->Friction;
								Restitution = PhysicalMaterial->Restitution;
								Density = PhysicalMaterial->Density;
							}
						}
					}
				}
			}
		}
	}
}
#endif

void UNiagaraDataInterfaceChaosDestruction::HandleCollisionEvents(const Chaos::FCollisionEventData& Event)
{
	ensure(IsInGameThread());
	Chaos::FCollisionDataArray const& CollisionDataIn = Event.CollisionData.AllCollisionsArray;

	CollisionEvents.Reset();

	// Copy data from Event into AllCollisionsArray
	// Also get Boundingbox related data and SurfaceType and save it as well
	CollisionEvents.AddUninitialized(Event.CollisionData.AllCollisionsArray.Num());

	int32 Idx = 0;
	for (Chaos::FCollidingData const& DataIn : CollisionDataIn)
	{
		auto& CopyData = CollisionEvents[Idx];

		CopyData = DataIn;

		// #GM: Disable this for now for perf
		/*
		GetMeshExtData(SolverData,
			AllCollisionsArray[Idx].ParticleIndexMesh == INDEX_NONE ? AllCollisionsArray[Idx].ParticleIndex : AllCollisionsArray[Idx].ParticleIndexMesh,
			PhysicsProxyReverseMappingArray,
			ParticleIndexReverseMappingArray,
			AllCollisionsArray[Idx].BoundingboxVolume,
			AllCollisionsArray[Idx].BoundingboxExtentMin,
			AllCollisionsArray[Idx].BoundingboxExtentMax,
			AllCollisionsArray[Idx].SurfaceType);
		*/

		UPhysicalMaterial* PhysicalMaterial = nullptr;
		int32 MaterialID = 0;

		UGeometryCollectionComponent* GeometryCollectionComponent = nullptr;
		UMaterialInterface* Material = nullptr;

		for (auto& Solver : Solvers)
		{
			if (Solver.PhysScene)
			{
				GeometryCollectionComponent = Solver.PhysScene->GetOwningComponent<UGeometryCollectionComponent>(DataIn.Proxy1);
				if (GeometryCollectionComponent)
					break;
			}
		}

		if (GeometryCollectionComponent)
		{
			PhysicalMaterial = GeometryCollectionComponent->GetPhysicalMaterial();
			ensure(PhysicalMaterial);
			if (PhysicalMaterial)
			{
				CopyData.SurfaceType1 = PhysicalMaterial->SurfaceType;
				CopyData.PhysicalMaterialName1 = PhysicalMaterial->GetFName();
			}
			else
			{
				CopyData.PhysicalMaterialName1 = FName();
			}
		}

		for (auto& Solver : Solvers)
		{
			if (Solver.PhysScene)
			{
				GeometryCollectionComponent = Solver.PhysScene->GetOwningComponent<UGeometryCollectionComponent>(DataIn.Proxy2);
				if (GeometryCollectionComponent)
					break;
			}
		}

		if (GeometryCollectionComponent)
		{
			PhysicalMaterial = GeometryCollectionComponent->GetPhysicalMaterial();
			ensure(PhysicalMaterial);
			if (PhysicalMaterial)
			{
				CopyData.SurfaceType2 = PhysicalMaterial->SurfaceType;
				CopyData.PhysicalMaterialName2 = PhysicalMaterial->GetFName();
			}
			else
			{
				CopyData.PhysicalMaterialName2 = FName();
			}
		}

		CopyData.BoundingboxVolume = 1000000.f;
		CopyData.BoundingboxExtentMin = 100.f;
		CopyData.BoundingboxExtentMax = 100.f;

		Idx++;
	}
}


void UNiagaraDataInterfaceChaosDestruction::FilterAllCollisions(TArray<Chaos::FCollidingDataExt>& AllCollisionsArray)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FilterAllCollisions);

	if (/*ParticleToProcess != nullptr ||*/
		bApplyMaterialsFilter ||
		ImpulseToSpawnMinMax.X > 0.f ||
		ImpulseToSpawnMinMax.Y > 0.f ||
		SpeedToSpawnMinMax.X > 0.f ||
		SpeedToSpawnMinMax.Y > 0.f ||
		MassToSpawnMinMax.X > 0.f ||
		MassToSpawnMinMax.Y > 0.f ||
		ExtentMinToSpawnMinMax.X > 0.f ||
		ExtentMinToSpawnMinMax.Y > 0.f ||
		ExtentMaxToSpawnMinMax.X > 0.f ||
		ExtentMaxToSpawnMinMax.Y > 0.f ||
		VolumeToSpawnMinMax.X > 0.f ||
		VolumeToSpawnMinMax.Y > 0.f ||
		SolverTimeToSpawnMinMax.X > 0.f ||
		SolverTimeToSpawnMinMax.Y > 0.f ||
		SurfaceTypeToSpawn != -1 ||
		LocationXToSpawn != ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_None ||
		LocationYToSpawn != ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_None ||
		LocationZToSpawn != ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_None)
	{
		TArray<Chaos::FCollidingDataExt> FilteredAllCollisionsArray;
		FilteredAllCollisionsArray.SetNumUninitialized(AllCollisionsArray.Num());

		int32 IdxFilteredCollisions = 0;

		float MinImpulseToSpawnSquared = ImpulseToSpawnMinMax.X * ImpulseToSpawnMinMax.X;
		float MaxImpulseToSpawnSquared = ImpulseToSpawnMinMax.Y * ImpulseToSpawnMinMax.Y;
		float MinSpeedToSpawnSquared = SpeedToSpawnMinMax.X * SpeedToSpawnMinMax.X;
		float MaxSpeedToSpawnSquared = SpeedToSpawnMinMax.Y * SpeedToSpawnMinMax.Y;

		auto IsMaterialInFilter = [&](const FName& InMaterialName)
		{
			if (!InMaterialName.IsValid())
			{
				return false;
			}

			for (const UPhysicalMaterial* Material : ChaosBreakingMaterialSet)
			{
				if (!Material)
				{
					continue;
				}

				if (Material->GetFName() == InMaterialName)
				{
					return true;
				}
			}

			return false;
		};

		for (int32 IdxCollision = 0; IdxCollision < AllCollisionsArray.Num(); ++IdxCollision)
		{
			float CollisionAccumulatedImpulseSquared = AllCollisionsArray[IdxCollision].AccumulatedImpulse.SizeSquared();
			float CollisionSpeedSquared = AllCollisionsArray[IdxCollision].Velocity1.SizeSquared();

			// @TODO(Chaos): Need to be optimized
			if (/*(ParticleToProcess != nullptr && AllCollisionsArrayInOut[IdxCollision].Particle != ParticleToProcess) ||*/
				(bApplyMaterialsFilter && !IsMaterialInFilter(AllCollisionsArray[IdxCollision].PhysicalMaterialName1) && !IsMaterialInFilter(AllCollisionsArray[IdxCollision].PhysicalMaterialName2)) ||
				(ImpulseToSpawnMinMax.X > 0.f && ImpulseToSpawnMinMax.Y < 0.f && CollisionAccumulatedImpulseSquared < MinImpulseToSpawnSquared) ||
				(ImpulseToSpawnMinMax.X < 0.f && ImpulseToSpawnMinMax.Y > 0.f && CollisionAccumulatedImpulseSquared > MaxImpulseToSpawnSquared) ||
				(ImpulseToSpawnMinMax.X > 0.f && ImpulseToSpawnMinMax.Y > 0.f && (CollisionAccumulatedImpulseSquared < MinImpulseToSpawnSquared || CollisionAccumulatedImpulseSquared > MaxImpulseToSpawnSquared)) ||
				(SpeedToSpawnMinMax.X > 0.f && SpeedToSpawnMinMax.Y < 0.f && CollisionSpeedSquared < MinSpeedToSpawnSquared) ||
				(SpeedToSpawnMinMax.X < 0.f && SpeedToSpawnMinMax.Y > 0.f && CollisionSpeedSquared > MaxSpeedToSpawnSquared) ||
				(SpeedToSpawnMinMax.X > 0.f && SpeedToSpawnMinMax.Y > 0.f && (CollisionSpeedSquared < MinSpeedToSpawnSquared || CollisionSpeedSquared > MaxSpeedToSpawnSquared)) ||
				(MassToSpawnMinMax.X > 0.f && MassToSpawnMinMax.Y < 0.f && AllCollisionsArray[IdxCollision].Mass1 < MassToSpawnMinMax.X) ||
				(MassToSpawnMinMax.X < 0.f && MassToSpawnMinMax.Y > 0.f && AllCollisionsArray[IdxCollision].Mass1 > MassToSpawnMinMax.Y) ||
				(MassToSpawnMinMax.X > 0.f && MassToSpawnMinMax.Y > 0.f && (AllCollisionsArray[IdxCollision].Mass1 < MassToSpawnMinMax.X || AllCollisionsArray[IdxCollision].Mass1 > MassToSpawnMinMax.Y)) ||
				(ExtentMinToSpawnMinMax.X > 0.f && ExtentMinToSpawnMinMax.Y < 0.f && AllCollisionsArray[IdxCollision].BoundingboxExtentMin < ExtentMinToSpawnMinMax.X) ||
				(ExtentMinToSpawnMinMax.X < 0.f && ExtentMinToSpawnMinMax.Y > 0.f && AllCollisionsArray[IdxCollision].BoundingboxExtentMin > ExtentMinToSpawnMinMax.Y) ||
				(ExtentMinToSpawnMinMax.X > 0.f && ExtentMinToSpawnMinMax.Y > 0.f && (AllCollisionsArray[IdxCollision].BoundingboxExtentMin < ExtentMinToSpawnMinMax.X || AllCollisionsArray[IdxCollision].BoundingboxExtentMin > ExtentMinToSpawnMinMax.Y)) ||
				(ExtentMaxToSpawnMinMax.X > 0.f && ExtentMaxToSpawnMinMax.Y < 0.f && AllCollisionsArray[IdxCollision].BoundingboxExtentMax < ExtentMaxToSpawnMinMax.X) ||
				(ExtentMaxToSpawnMinMax.X < 0.f && ExtentMaxToSpawnMinMax.Y > 0.f && AllCollisionsArray[IdxCollision].BoundingboxExtentMax > ExtentMaxToSpawnMinMax.Y) ||
				(ExtentMaxToSpawnMinMax.X > 0.f && ExtentMaxToSpawnMinMax.Y > 0.f && (AllCollisionsArray[IdxCollision].BoundingboxExtentMax < ExtentMaxToSpawnMinMax.X || AllCollisionsArray[IdxCollision].BoundingboxExtentMax > ExtentMaxToSpawnMinMax.Y)) ||
				(VolumeToSpawnMinMax.X > 0.f && VolumeToSpawnMinMax.Y < 0.f && AllCollisionsArray[IdxCollision].BoundingboxVolume < VolumeToSpawnMinMax.X) ||
				(VolumeToSpawnMinMax.X < 0.f && VolumeToSpawnMinMax.Y > 0.f && AllCollisionsArray[IdxCollision].BoundingboxVolume > VolumeToSpawnMinMax.Y) ||
				(VolumeToSpawnMinMax.X > 0.f && VolumeToSpawnMinMax.Y > 0.f && (AllCollisionsArray[IdxCollision].BoundingboxVolume < VolumeToSpawnMinMax.X || AllCollisionsArray[IdxCollision].BoundingboxVolume > VolumeToSpawnMinMax.Y)) ||
				(SolverTimeToSpawnMinMax.X > 0.f && SolverTimeToSpawnMinMax.Y < 0.f && SolverTime < SolverTimeToSpawnMinMax.X) ||
				(SolverTimeToSpawnMinMax.X < 0.f && SolverTimeToSpawnMinMax.Y > 0.f && SolverTime > SolverTimeToSpawnMinMax.Y) ||
				(SolverTimeToSpawnMinMax.X > 0.f && SolverTimeToSpawnMinMax.Y > 0.f && (SolverTime < SolverTimeToSpawnMinMax.X || SolverTime > SolverTimeToSpawnMinMax.Y)) ||
				(SurfaceTypeToSpawn != -1 && AllCollisionsArray[IdxCollision].SurfaceType1 != SurfaceTypeToSpawn && AllCollisionsArray[IdxCollision].SurfaceType2 != SurfaceTypeToSpawn) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Min && AllCollisionsArray[IdxCollision].Location.X <= LocationXToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Max && AllCollisionsArray[IdxCollision].Location.X >= LocationXToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_MinMax && (AllCollisionsArray[IdxCollision].Location.X <= LocationXToSpawnMinMax.X || AllCollisionsArray[IdxCollision].Location.X >= LocationXToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Min && AllCollisionsArray[IdxCollision].Location.Y <= LocationYToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Max && AllCollisionsArray[IdxCollision].Location.Y >= LocationYToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_MinMax && (AllCollisionsArray[IdxCollision].Location.Y <= LocationYToSpawnMinMax.X || AllCollisionsArray[IdxCollision].Location.Y >= LocationYToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Min && AllCollisionsArray[IdxCollision].Location.Z <= LocationZToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Max && AllCollisionsArray[IdxCollision].Location.Z >= LocationZToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_MinMax && (AllCollisionsArray[IdxCollision].Location.Z <= LocationZToSpawnMinMax.X || AllCollisionsArray[IdxCollision].Location.Z >= LocationZToSpawnMinMax.Y)) ||			
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Min && AllCollisionsArray[IdxCollision].Location.X > LocationXToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Max && AllCollisionsArray[IdxCollision].Location.X < LocationXToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_MinMax && !(AllCollisionsArray[IdxCollision].Location.X <= LocationXToSpawnMinMax.X || AllCollisionsArray[IdxCollision].Location.X >= LocationXToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Min && AllCollisionsArray[IdxCollision].Location.Y > LocationYToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Max && AllCollisionsArray[IdxCollision].Location.Y < LocationYToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_MinMax && !(AllCollisionsArray[IdxCollision].Location.Y <= LocationYToSpawnMinMax.X || AllCollisionsArray[IdxCollision].Location.Y >= LocationYToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Min && AllCollisionsArray[IdxCollision].Location.Z > LocationZToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Max && AllCollisionsArray[IdxCollision].Location.Z < LocationZToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_MinMax && !(AllCollisionsArray[IdxCollision].Location.Z <= LocationZToSpawnMinMax.X || AllCollisionsArray[IdxCollision].Location.Z >= LocationZToSpawnMinMax.Y)))
			{
				continue;
			}

			FilteredAllCollisionsArray[IdxFilteredCollisions] = AllCollisionsArray[IdxCollision];

			IdxFilteredCollisions++;
		}
		FilteredAllCollisionsArray.SetNum(IdxFilteredCollisions);

		// If collisions were filtered copy FilteredAllCollisionsArray back into AllCollisions
		if (FilteredAllCollisionsArray.Num() != AllCollisionsArray.Num())
		{
			AllCollisionsArray.SetNumUninitialized(FilteredAllCollisionsArray.Num());
		}
		for (int32 Idx = 0; Idx < FilteredAllCollisionsArray.Num(); ++Idx)
		{
			AllCollisionsArray[Idx] = FilteredAllCollisionsArray[Idx];
		}

		INC_DWORD_STAT_BY(STAT_NiagaraNumFilteredAllCollisions, FilteredAllCollisionsArray.Num());
	}
}

void UNiagaraDataInterfaceChaosDestruction::SortCollisions(TArray<Chaos::FCollidingDataExt>& CollisionsArray)
{
	SCOPE_CYCLE_COUNTER(STAT_CollisionCallbackSorting);

	if (DataSortingType == EDataSortTypeEnum::ChaosNiagara_DataSortType_SortByMassMaxToMin)
	{
		CollisionsArray.Sort(UNiagaraDataInterfaceChaosDestruction::CollisionDataSortByMassPredicateMaxToMin);
	}
	else if (DataSortingType == EDataSortTypeEnum::ChaosNiagara_DataSortType_SortByMassMinToMax)
	{
		CollisionsArray.Sort(UNiagaraDataInterfaceChaosDestruction::CollisionDataSortByMassPredicateMinToMax);
	}
	else if (DataSortingType == EDataSortTypeEnum::ChaosNiagara_DataSortType_RandomShuffle)
	{
		CollisionsArray.Sort(UNiagaraDataInterfaceChaosDestruction::CollisionDataRandomShuffleSortPredicate);
	}
}

void ComputeHashTable(const TArray<Chaos::FCollidingDataExt>& CollisionsArray, const FBox& SpatialHashVolume, const FVector& SpatialHashVolumeCellSize, const uint32 NumberOfCellsX, const uint32 NumberOfCellsY, const uint32 NumberOfCellsZ, TMultiMap<uint32, int32>& HashTableMap)
{
	FVector CellSizeInv(1.f / SpatialHashVolumeCellSize.X, 1.f / SpatialHashVolumeCellSize.Y, 1.f / SpatialHashVolumeCellSize.Z);

	// Create a Hash Table, but only store the cells with constraint(s) as a map HashTableMap<CellIdx, ConstraintIdx>
	uint32 NumberOfCellsXY = NumberOfCellsX * NumberOfCellsY;
	uint32 NumberOfCellsXYZ = NumberOfCellsXY * NumberOfCellsZ;

	for (int32 IdxCollision = 0; IdxCollision < CollisionsArray.Num(); ++IdxCollision)
	{
		FVector Location = (FVector)CollisionsArray[IdxCollision].Location;
		if (SpatialHashVolume.IsInsideOrOn(Location))
		{
			Location -= SpatialHashVolume.Min;
			uint32 HashTableIdx = (uint32)(Location.X * CellSizeInv.X) +
								  (uint32)(Location.Y * CellSizeInv.Y) * NumberOfCellsX +
								  (uint32)(Location.Z * CellSizeInv.Z) * NumberOfCellsXY;
			if (ensure(HashTableIdx < NumberOfCellsXYZ))
			{
				HashTableMap.Add(HashTableIdx, IdxCollision);
			}
		}
	}
}

void UNiagaraDataInterfaceChaosDestruction::GetCollisionsToSpawnFromCollisions(TArray<Chaos::FCollidingDataExt>& AllCollisionsArray,
	TArray<Chaos::FCollidingDataExt>& CollisionsToSpawnArray)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GetCollisionsToSpawnFromCollisions);

	const float SpatialHasVolumeExtentMin = 100.f;
	const float SpatialHasVolumeExtentMax = 1e8;

	if (DoSpatialHash &&
		(SpatialHashVolumeMax.X - SpatialHashVolumeMin.X) > SpatialHasVolumeExtentMin &&
		(SpatialHashVolumeMax.Y - SpatialHashVolumeMin.Y) > SpatialHasVolumeExtentMin &&
		(SpatialHashVolumeMax.Z - SpatialHashVolumeMin.Z) > SpatialHasVolumeExtentMin &&
		(SpatialHashVolumeMax.X - SpatialHashVolumeMin.X) < SpatialHasVolumeExtentMax &&
		(SpatialHashVolumeMax.Y - SpatialHashVolumeMin.Y) < SpatialHasVolumeExtentMax &&
		(SpatialHashVolumeMax.Z - SpatialHashVolumeMin.Z) < SpatialHasVolumeExtentMax &&
		SpatialHashVolumeCellSize.X >= 1.f && SpatialHashVolumeCellSize.Y >= 1.f && SpatialHashVolumeCellSize.Z >= 1.f &&
		AllCollisionsArray.Num() > 1)
	{
		// Adjust SpatialHashVolumeMin, SpatialHashVolumeMin based on SpatialHashVolumeCellSize
		uint32 NumberOfCellsX = FMath::CeilToInt((SpatialHashVolumeMax.X - SpatialHashVolumeMin.X) / SpatialHashVolumeCellSize.X);
		uint32 NumberOfCellsY = FMath::CeilToInt((SpatialHashVolumeMax.Y - SpatialHashVolumeMin.Y) / SpatialHashVolumeCellSize.Y);
		uint32 NumberOfCellsZ = FMath::CeilToInt((SpatialHashVolumeMax.Z - SpatialHashVolumeMin.Z) / SpatialHashVolumeCellSize.Z);

		float dX = ((float)NumberOfCellsX * SpatialHashVolumeCellSize.X - (SpatialHashVolumeMax.X - SpatialHashVolumeMin.X)) / 2.f;
		SpatialHashVolumeMin.X -= dX; SpatialHashVolumeMax.X += dX;
		float dY = ((float)NumberOfCellsY * SpatialHashVolumeCellSize.Y - (SpatialHashVolumeMax.Y - SpatialHashVolumeMin.Y)) / 2.f;
		SpatialHashVolumeMin.Y -= dY; SpatialHashVolumeMax.Y += dY;
		float dZ = ((float)NumberOfCellsZ * SpatialHashVolumeCellSize.Z - (SpatialHashVolumeMax.Z - SpatialHashVolumeMin.Z)) / 2.f;
		SpatialHashVolumeMin.Z -= dZ; SpatialHashVolumeMax.Z += dZ;

		FBox SpatialHashVolume(SpatialHashVolumeMin, SpatialHashVolumeMax);

		// Spatial hash the collisions
		TMultiMap<uint32, int32> HashTableMap;
		ComputeHashTable(AllCollisionsArray, SpatialHashVolume, SpatialHashVolumeCellSize, NumberOfCellsX, NumberOfCellsY, NumberOfCellsZ, HashTableMap);

		TArray<uint32> UsedCellsArray;
		HashTableMap.GetKeys(UsedCellsArray);

		for (int32 IdxCell = 0; IdxCell < UsedCellsArray.Num(); ++IdxCell)
		{
			TArray<int32> CollisionsInCellArray;
			HashTableMap.MultiFind(UsedCellsArray[IdxCell], CollisionsInCellArray);

			int32 NumCollisionsToGetFromCell = FMath::Min(MaxDataPerCell, CollisionsInCellArray.Num());
			for (int32 IdxCollision = 0; IdxCollision < NumCollisionsToGetFromCell; ++IdxCollision)
			{
				CollisionsToSpawnArray.Add(AllCollisionsArray[CollisionsInCellArray[IdxCollision]]);
			}
		}

		// CollisionsToSpawnArray has too many elements
		if (CollisionsToSpawnArray.Num() > MaxNumberOfDataEntriesToSpawn)
		{
			TArray<Chaos::FCollidingDataExt> CollisionsArray1;

			float FInc = (float)CollisionsToSpawnArray.Num() / (float)MaxNumberOfDataEntriesToSpawn;

			CollisionsArray1.SetNumUninitialized(MaxNumberOfDataEntriesToSpawn);
			for (int32 IdxCollision = 0; IdxCollision < MaxNumberOfDataEntriesToSpawn; ++IdxCollision)
			{
				int32 NewIdx = FMath::FloorToInt((float)IdxCollision * FInc);
				CollisionsArray1[IdxCollision] = CollisionsToSpawnArray[NewIdx];
			}

			CollisionsToSpawnArray.SetNumUninitialized(MaxNumberOfDataEntriesToSpawn);
			for (int32 IdxCollision = 0; IdxCollision < MaxNumberOfDataEntriesToSpawn; ++IdxCollision)
			{
				CollisionsToSpawnArray[IdxCollision] = CollisionsArray1[IdxCollision];
			}
		}
	}
	else
	{
		if (AllCollisionsArray.Num() <= MaxNumberOfDataEntriesToSpawn)
		{
			CollisionsToSpawnArray.SetNumUninitialized(AllCollisionsArray.Num());
			for (int32 IdxCollision = 0; IdxCollision < AllCollisionsArray.Num(); ++IdxCollision)
			{
				CollisionsToSpawnArray[IdxCollision] = AllCollisionsArray[IdxCollision];
			}
		}
		else
		{
			float FInc = (float)AllCollisionsArray.Num() / (float)MaxNumberOfDataEntriesToSpawn;

			CollisionsToSpawnArray.SetNumUninitialized(MaxNumberOfDataEntriesToSpawn);
			for (int32 IdxCollision = 0; IdxCollision < MaxNumberOfDataEntriesToSpawn; ++IdxCollision)
			{
				int32 NewIdx = FMath::FloorToInt((float)IdxCollision * FInc);
				CollisionsToSpawnArray[IdxCollision] = AllCollisionsArray[NewIdx];
			}
		}
	}

	INC_DWORD_STAT_BY(STAT_NiagaraNumCollisionsToSpawnParticles, CollisionsToSpawnArray.Num());
}

int32 UNiagaraDataInterfaceChaosDestruction::SpawnParticlesFromCollision(FSolverData SolverData,
																		 Chaos::FCollidingDataExt& Collision,
																		 FNDIChaosDestruction_InstanceData* InstData,
																		 float TimeData_MapsCreated,
																		 int32 IdxSolver)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SpawnParticlesFromCollision);
	float Latency = SolverData.Solver->GetSolverTime() - TimeData_MapsCreated;
	if (Latency <= MaxLatency)
	{
		int32 NumParticles = FMath::RandRange((int)SpawnMultiplierMinMax.X, (int)FMath::Max(SpawnMultiplierMinMax.X, SpawnMultiplierMinMax.Y));
		for (int32 Idx = 0; Idx < NumParticles; ++Idx)
		{
			if (FMath::FRand() > FMath::Clamp(SpawnChance, 0.f, 1.f))
			{
				continue;
			}

			// Compute random position
			float sign[2] = { -1.f, 1.f };
			float RandomX = sign[FMath::RandRange(0, 1)] * FMath::FRandRange(FMath::Min(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y), FMath::Max(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y));
			float RandomY = sign[FMath::RandRange(0, 1)] * FMath::FRandRange(FMath::Min(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y), FMath::Max(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y));
			float RandomZ = sign[FMath::RandRange(0, 1)] * FMath::FRandRange(FMath::Min(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y), FMath::Max(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y));
			FVector RandomPosition(RandomX, RandomY, RandomZ);

			// Compute velocity offset
			FVector VelocityOffset;
			VelocityOffset.X = FMath::FRandRange(FMath::Min(VelocityOffsetMin.X, VelocityOffsetMax.X), FMath::Max(VelocityOffsetMin.X, VelocityOffsetMax.X));
			VelocityOffset.Y = FMath::FRandRange(FMath::Min(VelocityOffsetMin.Y, VelocityOffsetMax.Y), FMath::Max(VelocityOffsetMin.Y, VelocityOffsetMax.Y));
			VelocityOffset.Z = FMath::FRandRange(FMath::Min(VelocityOffsetMin.Z, VelocityOffsetMax.Z), FMath::Max(VelocityOffsetMin.Z, VelocityOffsetMax.Z));

			// Compute random velocity using the selected Velocity Model
			FVector ParticleVelocity;
			if (RandomVelocityGenerationType == ERandomVelocityGenerationTypeEnum::ChaosNiagara_RandomVelocityGenerationType_RandomDistribution)
			{
				FVector RandomVector(FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f));
				RandomVector.Normalize();

				ParticleVelocity = RandomVector * FMath::FRandRange(RandomVelocityMagnitudeMinMax.X, FMath::Max(RandomVelocityMagnitudeMinMax.X, RandomVelocityMagnitudeMinMax.Y));
			}
			else if (RandomVelocityGenerationType == ERandomVelocityGenerationTypeEnum::ChaosNiagara_RandomVelocityGenerationType_CollisionNormalBased)
			{
				FVector RandomVector = FMath::VRandCone(Collision.Normal, FMath::DegreesToRadians(SpreadAngleMax));
				RandomVector.Normalize();

				ParticleVelocity = RandomVector * FMath::FRandRange(RandomVelocityMagnitudeMinMax.X, FMath::Max(RandomVelocityMagnitudeMinMax.X, RandomVelocityMagnitudeMinMax.Y));
			}
//			else if (RandomVelocityGenerationType == ERandomVelocityGenerationTypeEnum::ChaosNiagara_RandomVelocityGenerationType_NRandomSpread)
//			{
//			}
			else if (RandomVelocityGenerationType == ERandomVelocityGenerationTypeEnum::ChaosNiagara_RandomVelocityGenerationType_RandomDistributionWithStreamers)
			{
				FVector RandomVector(FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f));
				RandomVector.Normalize();

				ParticleVelocity = RandomVector * FMath::FRandRange(RandomVelocityMagnitudeMinMax.X, FMath::Max(RandomVelocityMagnitudeMinMax.X, RandomVelocityMagnitudeMinMax.Y));

				float RandomMultiplier = (FMath::FRand() < 0.2) ? 1.25f : 1.f;
				ParticleVelocity *= RandomMultiplier;
			}

			// Combine all velocity components
			FVector ComputedVelocity = (Collision.Velocity1 - Collision.Velocity2) * InheritedVelocityMultiplier + ParticleVelocity + VelocityOffset;

			// Clamp velocity
			if (FinalVelocityMagnitudeMinMax.X >= 0 && FinalVelocityMagnitudeMinMax.Y < 0)
			{
				if (ComputedVelocity.SizeSquared() < FinalVelocityMagnitudeMinMax.X * FinalVelocityMagnitudeMinMax.X)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.X;
				}
			}
			else if (FinalVelocityMagnitudeMinMax.X < 0 && FinalVelocityMagnitudeMinMax.Y >= 0)
			{
				if (ComputedVelocity.SizeSquared() > FinalVelocityMagnitudeMinMax.Y * FinalVelocityMagnitudeMinMax.Y)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.Y;
				}
			}
			else if (FinalVelocityMagnitudeMinMax.X >= 0 && FinalVelocityMagnitudeMinMax.Y >= 0)
			{
				if (ComputedVelocity.SizeSquared() < FinalVelocityMagnitudeMinMax.X * FinalVelocityMagnitudeMinMax.X)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.X;
				}
				else if (ComputedVelocity.SizeSquared() > FinalVelocityMagnitudeMinMax.Y * FinalVelocityMagnitudeMinMax.Y)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.Y;
				}
			}

			// Debug
			FVector ParticleColor = FVector::OneVector;
			if (DebugType == EDebugTypeEnum::ChaosNiagara_DebugType_ColorBySolver)
			{
				ParticleColor = ColorArray[IdxSolver % ColorArray.Num()];
			}
			else if (DebugType == EDebugTypeEnum::ChaosNiagara_DebugType_ColorByParticleIndex)
			{
				//ParticleColor = ColorArray[Collision.ParticleIndex % ColorArray.Num()];
			}

			// Store principal data
			InstData->PositionArray.Add(Collision.Location + RandomPosition);
			InstData->VelocityArray.Add(ComputedVelocity);
			InstData->SolverIDArray.Add(IdxSolver);

			// Store Collision data
			InstData->IncomingLocationArray.Add(Collision.Location);
			InstData->IncomingAccumulatedImpulseArray.Add(Collision.AccumulatedImpulse);
			InstData->IncomingNormalArray.Add(Collision.Normal);
			InstData->IncomingVelocity1Array.Add(Collision.Velocity1);
			InstData->IncomingVelocity2Array.Add(Collision.Velocity2);
			InstData->IncomingAngularVelocity1Array.Add(Collision.AngularVelocity1);
			InstData->IncomingAngularVelocity2Array.Add(Collision.AngularVelocity2);
			InstData->IncomingMass1Array.Add(Collision.Mass1);
			InstData->IncomingMass2Array.Add(Collision.Mass2);
			InstData->IncomingTimeArray.Add(TimeData_MapsCreated);

			// Store GeometryCollection data
			InstData->ExtentMinArray.Add(Collision.BoundingboxExtentMin);
			InstData->ExtentMaxArray.Add(Collision.BoundingboxExtentMax);
			InstData->VolumeArray.Add(Collision.BoundingboxVolume);
		}
	
		return NumParticles;
	}
	return 0;
}

bool UNiagaraDataInterfaceChaosDestruction::CollisionCallback(FNDIChaosDestruction_InstanceData* InstData)
{
	int32 IdxSolver = 0;
	for (FSolverData SolverData : Solvers)
	{
		if (SolverData.Solver->GetEventFilters()->IsCollisionEventEnabled() && CollisionEvents.Num() > 0 && SolverData.Solver->GetSolverTime() > 0.f && MaxNumberOfDataEntriesToSpawn > 0)
		{
			TArray<Chaos::FCollidingDataExt>& AllCollisionsArray = CollisionEvents;
			float TimeData_MapsCreated = SolverData.Solver->GetSolverTime();

#if STATS
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GatherMemoryStats);
				size_t SizeOfAllCollisions = sizeof(Chaos::FCollidingData) * AllCollisionsArray.Num();
				SET_MEMORY_STAT(STAT_AllCollisionsDataMemory, SizeOfAllCollisions);
			}

			INC_DWORD_STAT_BY(STAT_NiagaraNumAllCollisions, AllCollisionsArray.Num());
#endif // STATS


			if (AllCollisionsArray.Num() > 0)
			{
				// Filter AllCollisions
				// In case of filtering AllCollisions will be resized and filtered data will be copied back to AllCollisions
				FilterAllCollisions(AllCollisionsArray);

				// Sort AllCollisisons
				SortCollisions(AllCollisionsArray);

				// Get the collisions which will spawn particles
				TArray<Chaos::FCollidingDataExt> CollisionsToSpawnArray;

				GetCollisionsToSpawnFromCollisions(AllCollisionsArray, CollisionsToSpawnArray);

				// Spawn particles for collisions in CollisionsToSpawnArray
				for (int32 IdxCollision = 0; IdxCollision < CollisionsToSpawnArray.Num(); ++IdxCollision)
				{
					int32 NumParticlesSpawned = SpawnParticlesFromCollision(SolverData,
																			CollisionsToSpawnArray[IdxCollision],
																			InstData,
																			TimeData_MapsCreated,
																			IdxSolver);

					if (NumParticlesSpawned > 0)
					{
						// Get/Store Geometry/Physical Material data
						FLinearColor Color(1.f, 1.f, 1.f, 1.f);
						float Friction = 0.7f, Restitution = 0.3f, Density = 1.f;
						// #GM: Disable this for now for perf
						/*
						GetMesPhysicalData(SolverData,
							CollisionsToSpawnArray[IdxCollision].ParticleIndexMesh == INDEX_NONE ? CollisionsToSpawnArray[IdxCollision].ParticleIndex : CollisionsToSpawnArray[IdxCollision].ParticleIndexMesh,
							PhysicsProxyReverseMappingArray,
							ParticleIndexReverseMappingArray,
							Color,
							Friction,
							Restitution,
							Density);
						*/

						// jf: optimization: presize these arrays?
						for (int32 Idx = 0; Idx < NumParticlesSpawned; ++Idx)
						{
							InstData->SurfaceTypeArray.Add(CollisionsToSpawnArray[IdxCollision].SurfaceType1);
							InstData->ColorArray.Add(Color);
							InstData->FrictionArray.Add(Friction);
							InstData->RestitutionArray.Add(Restitution);
							InstData->DensityArray.Add(Density);
						}
					}
				}
			}
		}

		IdxSolver++;
	}

	INC_DWORD_STAT_BY(STAT_NiagaraNumParticlesSpawnedFromCollisions, InstData->PositionArray.Num());

	return false;
}

void UNiagaraDataInterfaceChaosDestruction::HandleBreakingEvents(const Chaos::FBreakingEventData& Event)
{
	ensure(IsInGameThread());
	Chaos::FBreakingDataArray const& BreakingDataIn = Event.BreakingData.AllBreakingsArray;

	BreakingEvents.Reset();

	// Copy data from Event
	BreakingEvents.AddZeroed(BreakingDataIn.Num());

	int32 Idx = 0;
	for (Chaos::FBreakingData const& DataIn : BreakingDataIn)
	{
		if (bGetExternalBreakingData)
		{
			auto& CopyData = BreakingEvents[Idx];
			CopyData = DataIn;

			Chaos::FRigidTransform3 Transform;

			// Ext Data..
			CopyData.TransformTranslation = Transform.GetTranslation();
			CopyData.TransformRotation = Transform.GetRotation();
			CopyData.TransformScale = Transform.GetScale3D();

			CopyData.BoundingBox = FBox(DataIn.BoundingBox.Min(), DataIn.BoundingBox.Max());
			FVector Extents = CopyData.BoundingBox.GetSize();
			CopyData.BoundingboxExtentMin = FMath::Min3(Extents[0], Extents[1], Extents[2]);
			CopyData.BoundingboxExtentMax = FMath::Max3(Extents[0], Extents[1], Extents[2]);
			CopyData.BoundingboxVolume = CopyData.BoundingBox.GetVolume();

			UPhysicalMaterial* PhysicalMaterial = nullptr;
			int32 MaterialID = 0;
			
			// #todo (bmiller) Need to have a better way to get at this data now that particle handle isn't in the Breaking event payload.
			/*
			if(DataIn.Particle && DataIn.Particle->Geometry() )
				MaterialID = DataIn.Particle->Geometry()->GetMaterialIndex(0);
			*/

			UGeometryCollectionComponent* GeometryCollectionComponent = nullptr;
			UMaterialInterface* Material = nullptr;

			for (auto& Solver : Solvers)
			{
				if (Solver.PhysScene)
				{
					GeometryCollectionComponent = Solver.PhysScene->GetOwningComponent<UGeometryCollectionComponent>(DataIn.Proxy);
					if (GeometryCollectionComponent)
						break;
				}
			}

			if (GeometryCollectionComponent)
			{
				PhysicalMaterial = GeometryCollectionComponent->GetPhysicalMaterial();
				ensure(PhysicalMaterial);
				if (PhysicalMaterial)
				{
					CopyData.SurfaceType = PhysicalMaterial->SurfaceType;
					CopyData.PhysicalMaterialName = PhysicalMaterial->GetFName();
				}
				else
				{
					CopyData.PhysicalMaterialName = FName();
				}
			}
		}
		else
		{
			BreakingEvents[Idx].BoundingboxVolume = 1000000.f;
			BreakingEvents[Idx].BoundingboxExtentMin = 100.0f;
			BreakingEvents[Idx].BoundingboxExtentMax = 100.0f;
			BreakingEvents[Idx].BoundingBox = FBox(FVector(-100.0f, -100.0f, -100.0f), FVector(100.0f, 100.0f, 100.0f));
			BreakingEvents[Idx].SurfaceType = 0;
			BreakingEvents[Idx].TransformTranslation = FVector(0.0f, 0.0f, 0.0f);
			BreakingEvents[Idx].TransformRotation = FQuat(0.0f, 0.0f, 0.0f, 1.0f);
			BreakingEvents[Idx].TransformScale = FVector(1.0f, 1.0f, 1.0f);
			BreakingEvents[Idx].PhysicalMaterialName = FName();
		}

		Idx++;
	}
}


void UNiagaraDataInterfaceChaosDestruction::FilterAllBreakings(TArray<Chaos::FBreakingDataExt>& AllBreakingsArray)
{
	if (bApplyMaterialsFilter || 
		SpeedToSpawnMinMax.X > 0.f ||
		SpeedToSpawnMinMax.Y > 0.f ||
		MassToSpawnMinMax.X > 0.f ||
		MassToSpawnMinMax.Y > 0.f ||
		ExtentMinToSpawnMinMax.X > 0.f ||
		ExtentMinToSpawnMinMax.Y > 0.f ||
		ExtentMaxToSpawnMinMax.X > 0.f ||
		ExtentMaxToSpawnMinMax.Y > 0.f ||
		VolumeToSpawnMinMax.X > 0.f ||
		VolumeToSpawnMinMax.Y > 0.f ||
		SolverTimeToSpawnMinMax.X > 0.f ||
		SolverTimeToSpawnMinMax.Y > 0.f ||
		SurfaceTypeToSpawn != -1 ||
		LocationXToSpawn != ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_None ||
		LocationYToSpawn != ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_None ||
		LocationZToSpawn != ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_None)
	{ 
		TArray<Chaos::FBreakingDataExt> FilteredAllBreakingsArray;
		FilteredAllBreakingsArray.SetNumUninitialized(AllBreakingsArray.Num());

		int32 IdxFilteredBreakings = 0;

		float MinSpeedToSpawnSquared = SpeedToSpawnMinMax.X * SpeedToSpawnMinMax.X;
		float MaxSpeedToSpawnSquared = SpeedToSpawnMinMax.Y * SpeedToSpawnMinMax.Y;

		auto IsMaterialInFilter = [&](const FName& InMaterialName) {
			if (!InMaterialName.IsValid())
			{
				return false;
			}

			for (const UPhysicalMaterial* Material : ChaosBreakingMaterialSet)
			{
				if (!Material)
				{
					continue;
				}

				if (Material->GetFName() == InMaterialName)
				{
					return true;
				}
			}

			return false;
		};

		for (int32 IdxBreaking = 0; IdxBreaking < AllBreakingsArray.Num(); ++IdxBreaking)
		{
			float BreakingSpeedSquared = AllBreakingsArray[IdxBreaking].Velocity.SizeSquared();

			if ((bApplyMaterialsFilter && !IsMaterialInFilter(AllBreakingsArray[IdxBreaking].PhysicalMaterialName)) ||
				(SpeedToSpawnMinMax.X > 0.f && SpeedToSpawnMinMax.Y < 0.f && BreakingSpeedSquared < MinSpeedToSpawnSquared) ||
				(SpeedToSpawnMinMax.X < 0.f && SpeedToSpawnMinMax.Y > 0.f && BreakingSpeedSquared > MaxSpeedToSpawnSquared) ||
				(SpeedToSpawnMinMax.X > 0.f && SpeedToSpawnMinMax.Y > 0.f && (BreakingSpeedSquared < MinSpeedToSpawnSquared || BreakingSpeedSquared > MaxSpeedToSpawnSquared)) ||
				(MassToSpawnMinMax.X > 0.f && MassToSpawnMinMax.Y < 0.f && AllBreakingsArray[IdxBreaking].Mass < MassToSpawnMinMax.X) ||
				(MassToSpawnMinMax.X < 0.f && MassToSpawnMinMax.Y > 0.f && AllBreakingsArray[IdxBreaking].Mass > MassToSpawnMinMax.Y) ||
				(MassToSpawnMinMax.X > 0.f && MassToSpawnMinMax.Y > 0.f && (AllBreakingsArray[IdxBreaking].Mass < MassToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].Mass > MassToSpawnMinMax.Y)) ||
				(ExtentMinToSpawnMinMax.X > 0.f && ExtentMinToSpawnMinMax.Y < 0.f && AllBreakingsArray[IdxBreaking].BoundingboxExtentMin < ExtentMinToSpawnMinMax.X) ||
				(ExtentMinToSpawnMinMax.X < 0.f && ExtentMinToSpawnMinMax.Y > 0.f && AllBreakingsArray[IdxBreaking].BoundingboxExtentMin > ExtentMinToSpawnMinMax.Y) ||
				(ExtentMinToSpawnMinMax.X > 0.f && ExtentMinToSpawnMinMax.Y > 0.f && (AllBreakingsArray[IdxBreaking].BoundingboxExtentMin < ExtentMinToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].BoundingboxExtentMin > ExtentMinToSpawnMinMax.Y)) ||
				(ExtentMaxToSpawnMinMax.X > 0.f && ExtentMaxToSpawnMinMax.Y < 0.f && AllBreakingsArray[IdxBreaking].BoundingboxExtentMax < ExtentMaxToSpawnMinMax.X) ||
				(ExtentMaxToSpawnMinMax.X < 0.f && ExtentMaxToSpawnMinMax.Y > 0.f && AllBreakingsArray[IdxBreaking].BoundingboxExtentMax > ExtentMaxToSpawnMinMax.Y) ||
				(ExtentMaxToSpawnMinMax.X > 0.f && ExtentMaxToSpawnMinMax.Y > 0.f && (AllBreakingsArray[IdxBreaking].BoundingboxExtentMax < ExtentMaxToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].BoundingboxExtentMax > ExtentMaxToSpawnMinMax.Y)) ||
				(VolumeToSpawnMinMax.X > 0.f && VolumeToSpawnMinMax.Y < 0.f && AllBreakingsArray[IdxBreaking].BoundingboxVolume < VolumeToSpawnMinMax.X) ||
				(VolumeToSpawnMinMax.X < 0.f && VolumeToSpawnMinMax.Y > 0.f && AllBreakingsArray[IdxBreaking].BoundingboxVolume > VolumeToSpawnMinMax.Y) ||
				(VolumeToSpawnMinMax.X > 0.f && VolumeToSpawnMinMax.Y > 0.f && (AllBreakingsArray[IdxBreaking].BoundingboxVolume < VolumeToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].BoundingboxVolume > VolumeToSpawnMinMax.Y)) ||
				(SolverTimeToSpawnMinMax.X > 0.f && SolverTimeToSpawnMinMax.Y < 0.f && SolverTime < SolverTimeToSpawnMinMax.X) ||
				(SolverTimeToSpawnMinMax.X < 0.f && SolverTimeToSpawnMinMax.Y > 0.f && SolverTime > SolverTimeToSpawnMinMax.Y) ||
				(SolverTimeToSpawnMinMax.X > 0.f && SolverTimeToSpawnMinMax.Y > 0.f && (SolverTime < SolverTimeToSpawnMinMax.X || SolverTime > SolverTimeToSpawnMinMax.Y)) ||
				(SurfaceTypeToSpawn != -1 && AllBreakingsArray[IdxBreaking].SurfaceType != SurfaceTypeToSpawn) ||				
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Min && AllBreakingsArray[IdxBreaking].Location.X <= LocationXToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Max && AllBreakingsArray[IdxBreaking].Location.X >= LocationXToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_MinMax && (AllBreakingsArray[IdxBreaking].Location.X <= LocationXToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].Location.X >= LocationXToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Min && AllBreakingsArray[IdxBreaking].Location.Y <= LocationYToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Max && AllBreakingsArray[IdxBreaking].Location.Y >= LocationYToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_MinMax && (AllBreakingsArray[IdxBreaking].Location.Y <= LocationYToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].Location.Y >= LocationYToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Min && AllBreakingsArray[IdxBreaking].Location.Z <= LocationZToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Max && AllBreakingsArray[IdxBreaking].Location.Z >= LocationZToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_MinMax && (AllBreakingsArray[IdxBreaking].Location.Z <= LocationZToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].Location.Z >= LocationZToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Min && AllBreakingsArray[IdxBreaking].Location.X > LocationXToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Max && AllBreakingsArray[IdxBreaking].Location.X < LocationXToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_MinMax && !(AllBreakingsArray[IdxBreaking].Location.X <= LocationXToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].Location.X >= LocationXToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Min && AllBreakingsArray[IdxBreaking].Location.Y > LocationYToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Max && AllBreakingsArray[IdxBreaking].Location.Y < LocationYToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_MinMax && !(AllBreakingsArray[IdxBreaking].Location.Y <= LocationYToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].Location.Y >= LocationYToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Min && AllBreakingsArray[IdxBreaking].Location.Z > LocationZToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Max && AllBreakingsArray[IdxBreaking].Location.Z < LocationZToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_MinMax && !(AllBreakingsArray[IdxBreaking].Location.Z <= LocationZToSpawnMinMax.X || AllBreakingsArray[IdxBreaking].Location.Z >= LocationZToSpawnMinMax.Y)))
			{
				continue;
			}

			FilteredAllBreakingsArray[IdxFilteredBreakings] = AllBreakingsArray[IdxBreaking];

			IdxFilteredBreakings++;
		}
		FilteredAllBreakingsArray.SetNum(IdxFilteredBreakings);

		// If Breakings were filtered copy FilteredAllBreakingsArray back into AllBreakings
		if (FilteredAllBreakingsArray.Num() != AllBreakingsArray.Num())
		{
			AllBreakingsArray.SetNumUninitialized(FilteredAllBreakingsArray.Num());
		}
		for (int32 Idx = 0; Idx < FilteredAllBreakingsArray.Num(); ++Idx)
		{
			AllBreakingsArray[Idx] = FilteredAllBreakingsArray[Idx];
		}

		INC_DWORD_STAT_BY(STAT_NiagaraNumFilteredAllBreakings, FilteredAllBreakingsArray.Num());
	}
}

void UNiagaraDataInterfaceChaosDestruction::SortBreakings(TArray<Chaos::FBreakingDataExt>& BreakingsArray)
{
	SCOPE_CYCLE_COUNTER(STAT_BreakingCallbackSorting);

	if (DataSortingType == EDataSortTypeEnum::ChaosNiagara_DataSortType_SortByMassMaxToMin)
	{
		BreakingsArray.Sort(UNiagaraDataInterfaceChaosDestruction::BreakingDataSortByMassPredicateMaxToMin);
	}
	else if (DataSortingType == EDataSortTypeEnum::ChaosNiagara_DataSortType_SortByMassMinToMax)
	{
		BreakingsArray.Sort(UNiagaraDataInterfaceChaosDestruction::BreakingDataSortByMassPredicateMinToMax);
	}
	else if (DataSortingType == EDataSortTypeEnum::ChaosNiagara_DataSortType_RandomShuffle)
	{
		BreakingsArray.Sort(UNiagaraDataInterfaceChaosDestruction::BreakingDataRandomShuffleSortPredicate);
	}
}

void ComputeHashTable(const TArray<Chaos::FBreakingDataExt>& BreakingsArray, const FBox& SpatialHashVolume, const FVector& SpatialHashVolumeCellSize, const uint32 NumberOfCellsX, const uint32 NumberOfCellsY, const uint32 NumberOfCellsZ, TMultiMap<uint32, int32>& HashTableMap)
{
	FVector CellSizeInv(1.f / SpatialHashVolumeCellSize.X, 1.f / SpatialHashVolumeCellSize.Y, 1.f / SpatialHashVolumeCellSize.Z);

	// Create a Hash Table, but only store the cells with constraint(s) as a map HashTableMap<CellIdx, BreakingIdx>
	uint32 NumberOfCellsXY = NumberOfCellsX * NumberOfCellsY;
	uint32 NumberOfCellsXYZ = NumberOfCellsXY * NumberOfCellsZ;

	for (int32 IdxBreaking = 0; IdxBreaking < BreakingsArray.Num(); ++IdxBreaking)
	{
		FVector Location = (FVector)BreakingsArray[IdxBreaking].Location;
		if (SpatialHashVolume.IsInsideOrOn(Location))
		{
			Location -= SpatialHashVolume.Min;
			uint32 HashTableIdx = (uint32)(Location.X * CellSizeInv.X) +
								  (uint32)(Location.Y * CellSizeInv.Y) * NumberOfCellsX +
								  (uint32)(Location.Z * CellSizeInv.Z) * NumberOfCellsXY;
			if (ensure(HashTableIdx < NumberOfCellsXYZ))
			{
				HashTableMap.Add(HashTableIdx, IdxBreaking);
			}
		}
	}
}

void UNiagaraDataInterfaceChaosDestruction::GetBreakingsToSpawnFromBreakings(TArray<Chaos::FBreakingDataExt>& AllBreakingsArray,
																			 TArray<Chaos::FBreakingDataExt>& BreakingsToSpawnArray)
{
	const float SpatialHasVolumeExtentMin = 100.f;
	const float SpatialHasVolumeExtentMax = 1e8;

	if (DoSpatialHash &&
		(SpatialHashVolumeMax.X - SpatialHashVolumeMin.X) > SpatialHasVolumeExtentMin &&
		(SpatialHashVolumeMax.Y - SpatialHashVolumeMin.Y) > SpatialHasVolumeExtentMin &&
		(SpatialHashVolumeMax.Z - SpatialHashVolumeMin.Z) > SpatialHasVolumeExtentMin &&
		(SpatialHashVolumeMax.X - SpatialHashVolumeMin.X) < SpatialHasVolumeExtentMax &&
		(SpatialHashVolumeMax.Y - SpatialHashVolumeMin.Y) < SpatialHasVolumeExtentMax &&
		(SpatialHashVolumeMax.Z - SpatialHashVolumeMin.Z) < SpatialHasVolumeExtentMax &&
		SpatialHashVolumeCellSize.X >= 1.f && SpatialHashVolumeCellSize.Y >= 1.f && SpatialHashVolumeCellSize.Z >= 1.f &&
		AllBreakingsArray.Num() > 1)
	{
		// Adjust SpatialHashVolumeMin, SpatialHashVolumeMin based on SpatialHashVolumeCellSize
		uint32 NumberOfCellsX = FMath::CeilToInt((SpatialHashVolumeMax.X - SpatialHashVolumeMin.X) / SpatialHashVolumeCellSize.X);
		uint32 NumberOfCellsY = FMath::CeilToInt((SpatialHashVolumeMax.Y - SpatialHashVolumeMin.Y) / SpatialHashVolumeCellSize.Y);
		uint32 NumberOfCellsZ = FMath::CeilToInt((SpatialHashVolumeMax.Z - SpatialHashVolumeMin.Z) / SpatialHashVolumeCellSize.Z);

		float dX = ((float)NumberOfCellsX * SpatialHashVolumeCellSize.X - (SpatialHashVolumeMax.X - SpatialHashVolumeMin.X)) / 2.f;
		SpatialHashVolumeMin.X -= dX; SpatialHashVolumeMax.X += dX;
		float dY = ((float)NumberOfCellsY * SpatialHashVolumeCellSize.Y - (SpatialHashVolumeMax.Y - SpatialHashVolumeMin.Y)) / 2.f;
		SpatialHashVolumeMin.Y -= dY; SpatialHashVolumeMax.Y += dY;
		float dZ = ((float)NumberOfCellsZ * SpatialHashVolumeCellSize.Z - (SpatialHashVolumeMax.Z - SpatialHashVolumeMin.Z)) / 2.f;
		SpatialHashVolumeMin.Z -= dZ; SpatialHashVolumeMax.Z += dZ;

		FBox SpatialHashVolume(SpatialHashVolumeMin, SpatialHashVolumeMax);

		// Spatial hash the Breakings
		TMultiMap<uint32, int32> HashTableMap;
		ComputeHashTable(AllBreakingsArray, SpatialHashVolume, SpatialHashVolumeCellSize, NumberOfCellsX, NumberOfCellsY, NumberOfCellsZ, HashTableMap);

		TArray<uint32> UsedCellsArray;
		HashTableMap.GetKeys(UsedCellsArray);

		for (int32 IdxCell = 0; IdxCell < UsedCellsArray.Num(); ++IdxCell)
		{
			TArray<int32> BreakingsInCellArray;
			HashTableMap.MultiFind(UsedCellsArray[IdxCell], BreakingsInCellArray);

			int32 NumBreakingsToGetFromCell = FMath::Min(MaxDataPerCell, BreakingsInCellArray.Num());
			for (int32 IdxBreaking = 0; IdxBreaking < NumBreakingsToGetFromCell; ++IdxBreaking)
			{
				BreakingsToSpawnArray.Add(AllBreakingsArray[BreakingsInCellArray[IdxBreaking]]);
			}
		}

		// BreakingsToSpawnArray has too many elements
		if (BreakingsToSpawnArray.Num() > MaxNumberOfDataEntriesToSpawn)
		{
			TArray<Chaos::FBreakingDataExt> BreakingsArray1;

			float FInc = (float)BreakingsToSpawnArray.Num() / (float)MaxNumberOfDataEntriesToSpawn;

			BreakingsArray1.SetNumUninitialized(MaxNumberOfDataEntriesToSpawn);
			for (int32 IdxBreaking = 0; IdxBreaking < MaxNumberOfDataEntriesToSpawn; ++IdxBreaking)
			{
				int32 NewIdx = FMath::FloorToInt((float)IdxBreaking * FInc);
				BreakingsArray1[IdxBreaking] = BreakingsToSpawnArray[NewIdx];
			}

			BreakingsToSpawnArray.SetNumUninitialized(MaxNumberOfDataEntriesToSpawn);
			for (int32 IdxBreaking = 0; IdxBreaking < MaxNumberOfDataEntriesToSpawn; ++IdxBreaking)
			{
				BreakingsToSpawnArray[IdxBreaking] = BreakingsArray1[IdxBreaking];
			}
		}
	}
	else
	{
		if (AllBreakingsArray.Num() <= MaxNumberOfDataEntriesToSpawn)
		{
			BreakingsToSpawnArray.SetNumUninitialized(AllBreakingsArray.Num());
			for (int32 IdxBreaking = 0; IdxBreaking < AllBreakingsArray.Num(); ++IdxBreaking)
			{
				BreakingsToSpawnArray[IdxBreaking] = AllBreakingsArray[IdxBreaking];
			}
		}
		else
		{
			float FInc = (float)AllBreakingsArray.Num() / (float)MaxNumberOfDataEntriesToSpawn;

			BreakingsToSpawnArray.SetNumUninitialized(MaxNumberOfDataEntriesToSpawn);
			for (int32 IdxBreaking = 0; IdxBreaking < MaxNumberOfDataEntriesToSpawn; ++IdxBreaking)
			{
				int32 NewIdx = FMath::FloorToInt((float)IdxBreaking * FInc);
				BreakingsToSpawnArray[IdxBreaking] = AllBreakingsArray[NewIdx];
			}
		}
	}

	INC_DWORD_STAT_BY(STAT_NiagaraNumBreakingsToSpawnParticles, BreakingsToSpawnArray.Num());
}

int32 UNiagaraDataInterfaceChaosDestruction::SpawnParticlesFromBreaking(FSolverData SolverData,
																		Chaos::FBreakingDataExt& Breaking,
																		FNDIChaosDestruction_InstanceData* InstData,
																		float TimeData_MapsCreated,
																		int32 IdxSolver)
{
	float Latency = SolverData.Solver->GetSolverTime() - TimeData_MapsCreated;
	if (Latency <= MaxLatency)
	{
		int32 NumParticles = FMath::RandRange((int)SpawnMultiplierMinMax.X, (int)FMath::Max(SpawnMultiplierMinMax.X, SpawnMultiplierMinMax.Y));
		for (int32 Idx = 0; Idx < NumParticles; ++Idx)
		{
			if (FMath::FRand() > FMath::Clamp(SpawnChance, 0.f, 1.f))
			{
				continue;
			}

			// Compute random position
			float sign[2] = { -1.f, 1.f };
			float RandomX = sign[FMath::RandRange(0, 1)] * FMath::FRandRange(FMath::Min(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y), FMath::Max(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y));
			float RandomY = sign[FMath::RandRange(0, 1)] * FMath::FRandRange(FMath::Min(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y), FMath::Max(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y));
			float RandomZ = sign[FMath::RandRange(0, 1)] * FMath::FRandRange(FMath::Min(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y), FMath::Max(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y));
			FVector RandomPosition(RandomX, RandomY, RandomZ);

			// Compute velocity offset
			FVector VelocityOffset;
			VelocityOffset.X = FMath::FRandRange(FMath::Min(VelocityOffsetMin.X, VelocityOffsetMax.X), FMath::Max(VelocityOffsetMin.X, VelocityOffsetMax.X));
			VelocityOffset.Y = FMath::FRandRange(FMath::Min(VelocityOffsetMin.Y, VelocityOffsetMax.Y), FMath::Max(VelocityOffsetMin.Y, VelocityOffsetMax.Y));
			VelocityOffset.Z = FMath::FRandRange(FMath::Min(VelocityOffsetMin.Z, VelocityOffsetMax.Z), FMath::Max(VelocityOffsetMin.Z, VelocityOffsetMax.Z));

			// Compute random velocity using the selected Velocity Model
			FVector ParticleVelocity;
			if (RandomVelocityGenerationType == ERandomVelocityGenerationTypeEnum::ChaosNiagara_RandomVelocityGenerationType_RandomDistribution)
			{
				FVector RandomVector(FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f));
				RandomVector.Normalize();

				ParticleVelocity = RandomVector * FMath::FRandRange(RandomVelocityMagnitudeMinMax.X, FMath::Max(RandomVelocityMagnitudeMinMax.X, RandomVelocityMagnitudeMinMax.Y));
			}
			else if (RandomVelocityGenerationType == ERandomVelocityGenerationTypeEnum::ChaosNiagara_RandomVelocityGenerationType_RandomDistributionWithStreamers)
			{
				FVector RandomVector(FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f));
				RandomVector.Normalize();

				ParticleVelocity = RandomVector * FMath::FRandRange(RandomVelocityMagnitudeMinMax.X, FMath::Max(RandomVelocityMagnitudeMinMax.X, RandomVelocityMagnitudeMinMax.Y));

				float RandomMultiplier = (FMath::FRand() < 0.2) ? 1.25f : 1.f;
				ParticleVelocity *= RandomMultiplier;
			}

			// Combine all velocity components
			FVector ComputedVelocity = Breaking.Velocity * InheritedVelocityMultiplier + ParticleVelocity + VelocityOffset;

			// Clamp velocity
			if (FinalVelocityMagnitudeMinMax.X >= 0 && FinalVelocityMagnitudeMinMax.Y < 0)
			{
				if (ComputedVelocity.SizeSquared() < FinalVelocityMagnitudeMinMax.X * FinalVelocityMagnitudeMinMax.X)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.X;
				}
			}
			else if (FinalVelocityMagnitudeMinMax.X < 0 && FinalVelocityMagnitudeMinMax.Y >= 0)
			{
				if (ComputedVelocity.SizeSquared() > FinalVelocityMagnitudeMinMax.Y * FinalVelocityMagnitudeMinMax.Y)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.Y;
				}
			}
			else if (FinalVelocityMagnitudeMinMax.X >= 0 && FinalVelocityMagnitudeMinMax.Y >= 0)
			{
				if (ComputedVelocity.SizeSquared() < FinalVelocityMagnitudeMinMax.X * FinalVelocityMagnitudeMinMax.X)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.X;
				}
				else if (ComputedVelocity.SizeSquared() > FinalVelocityMagnitudeMinMax.Y * FinalVelocityMagnitudeMinMax.Y)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.Y;
				}
			}

			// Debug
			FVector ParticleColor = FVector::OneVector;
			if (DebugType == EDebugTypeEnum::ChaosNiagara_DebugType_ColorBySolver)
			{
				ParticleColor = ColorArray[IdxSolver % ColorArray.Num()];
			}
			else if (DebugType == EDebugTypeEnum::ChaosNiagara_DebugType_ColorByParticleIndex)
			{
				//ParticleColor = ColorArray[Breaking.ParticleIndex % ColorArray.Num()]; #todo: ParticleIndex no longer exits
			}

			// Store principal data
			InstData->PositionArray.Add(Breaking.Location + RandomPosition);
			InstData->VelocityArray.Add(ComputedVelocity);
			InstData->SolverIDArray.Add(IdxSolver);

			// Store Breaking data
			InstData->IncomingLocationArray.Add(Breaking.Location);
			InstData->IncomingVelocity1Array.Add(Breaking.Velocity);
			InstData->IncomingAngularVelocity1Array.Add(Breaking.AngularVelocity);
			InstData->IncomingMass1Array.Add(Breaking.Mass);
			InstData->IncomingTimeArray.Add(TimeData_MapsCreated);

			// Store GeometryCollection data
			InstData->ExtentMinArray.Add(Breaking.BoundingboxExtentMin);
			InstData->ExtentMaxArray.Add(Breaking.BoundingboxExtentMax);
			InstData->VolumeArray.Add(Breaking.BoundingboxVolume);
			InstData->BoundsArray.Add(Breaking.BoundingBox.Max - Breaking.BoundingBox.Min);

			// Set not related to default
			InstData->IncomingAccumulatedImpulseArray.Add(FVector(ForceInitToZero));
			InstData->IncomingNormalArray.Add(FVector(ForceInitToZero));
			InstData->IncomingVelocity2Array.Add(FVector(ForceInitToZero));
			InstData->IncomingAngularVelocity2Array.Add(FVector(ForceInitToZero));
			InstData->IncomingMass2Array.Add(0.f);
		}

		return NumParticles;
	}

	return 0;
}

bool UNiagaraDataInterfaceChaosDestruction::BreakingCallback(FNDIChaosDestruction_InstanceData* InstData)
{
	int32 IdxSolver = 0;
	for (FSolverData SolverData : Solvers)
	{
		if (SolverData.Solver->GetEventFilters()->IsBreakingEventEnabled() && BreakingEvents.Num() > 0 && SolverData.Solver->GetSolverTime() > 0.f && MaxNumberOfDataEntriesToSpawn > 0)
		{
			TArray<Chaos::FBreakingDataExt>& AllBreakingsArray = BreakingEvents;
			float TimeData_MapsCreated = SolverData.Solver->GetSolverTime();

			{
				size_t SizeOfAllBreakings = sizeof(Chaos::FBreakingData) * AllBreakingsArray.Num();
				size_t SizeOfAllBreakingsIndicesByPhysicsProxy = 0;
				SET_MEMORY_STAT(STAT_AllBreakingsDataMemory, SizeOfAllBreakings);
				SET_MEMORY_STAT(STAT_AllBreakingsIndicesByPhysicsProxyMemory, SizeOfAllBreakingsIndicesByPhysicsProxy);
			}

			INC_DWORD_STAT_BY(STAT_NiagaraNumAllBreakings, AllBreakingsArray.Num());

			if (AllBreakingsArray.Num() > 0)
			{
				// Filter AllBreakings
				// In case of filtering AllBreakings will be resized and filtered data will be copied back to AllBreakings
				FilterAllBreakings(AllBreakingsArray);

				// Sort AllCollisisons
				SortBreakings(AllBreakingsArray);

				// Get the Breakings which will spawn particles
				TArray<Chaos::FBreakingDataExt> BreakingsToSpawnArray;

				GetBreakingsToSpawnFromBreakings(AllBreakingsArray, BreakingsToSpawnArray);

				// Spawn particles for Breakings in BreakingsToSpawnArray
				for (int32 IdxBreaking = 0; IdxBreaking < BreakingsToSpawnArray.Num(); ++IdxBreaking)
				{
					int32 NumParticlesSpawned = SpawnParticlesFromBreaking(SolverData,
						BreakingsToSpawnArray[IdxBreaking],
						InstData,
						TimeData_MapsCreated,
						IdxSolver);

					if (NumParticlesSpawned > 0)
					{
						// Get/Store Geometry/Physical Material data
						
						FLinearColor Color(1.f, 1.f, 1.f, 1.f);
						float Friction = 0.7f, Restitution = 0.3f, Density = 1.f;
						if (bGetExternalBreakingData)
						{
							/* #GM: Disable this for now for perf
							GetMesPhysicalData(SolverData,
											   0, // #todo: BreakingsToSpawnArray[IdxBreaking].ParticleIndexMesh == INDEX_NONE ? BreakingsToSpawnArray[IdxBreaking].ParticleIndex : BreakingsToSpawnArray[IdxBreaking].ParticleIndexMesh,
											   PhysicsProxyReverseMappingArray,
											   ParticleIndexReverseMappingArray,
											   Color,
											   Friction,
											   Restitution,
											   Density);*/
						}
						
						for (int32 Idx = 0; Idx < NumParticlesSpawned; ++Idx)
						{
							InstData->SurfaceTypeArray.Add(BreakingsToSpawnArray[IdxBreaking].SurfaceType);
							InstData->TransformTranslationArray.Add(BreakingsToSpawnArray[IdxBreaking].TransformTranslation);
							InstData->TransformRotationArray.Add(BreakingsToSpawnArray[IdxBreaking].TransformRotation);
							InstData->TransformScaleArray.Add(BreakingsToSpawnArray[IdxBreaking].TransformScale);
							InstData->ColorArray.Add(Color);
							InstData->FrictionArray.Add(Friction);
							InstData->RestitutionArray.Add(Restitution);
							InstData->DensityArray.Add(Density);
						}
					}
				}
			}
		}

		IdxSolver++;
	}

	INC_DWORD_STAT_BY(STAT_NiagaraNumParticlesSpawnedFromBreakings, InstData->PositionArray.Num());

	return false;

}

void UNiagaraDataInterfaceChaosDestruction::HandleTrailingEvents(const Chaos::FTrailingEventData& Event)
{
	ensure(IsInGameThread());

	TrailingEvents.Reset();

	const float TrailMinSpeedToSpawnSquared = TrailMinSpeedToSpawn * TrailMinSpeedToSpawn;

	auto IsMaterialInFilter = [&](const FName& InMaterialName) 
	{
		if (!InMaterialName.IsValid())
		{
			return false;
		}

		for (const UPhysicalMaterial* Material : ChaosBreakingMaterialSet)
		{
			if (!Material)
			{
				continue;
			}

			if (Material->GetFName() == InMaterialName)
			{
				return true;
			}
		}

		return false;
	};

	auto GetGeometryCollectionComponentFromProxy = 
		[this](IPhysicsProxyBase* InProxy) -> UGeometryCollectionComponent*
		{
			for (auto& Solver : Solvers)
			{
				if (Solver.PhysScene)
				{
					if (UGeometryCollectionComponent* GeometryCollectionComponent = Solver.PhysScene->GetOwningComponent<UGeometryCollectionComponent>(InProxy))
					{
						return GeometryCollectionComponent;
					}
				}
			}
			return nullptr;
		};

	const Chaos::FTrailingDataArray& InEvents = Event.TrailingData.AllTrailingsArray;

	for (const Chaos::FTrailingData& InEvent : InEvents)
	{
		if (InEvent.Velocity.SquaredLength() >= TrailMinSpeedToSpawnSquared)
		{
			if (UGeometryCollectionComponent* GeometryCollectionComponent = GetGeometryCollectionComponentFromProxy(InEvent.Proxy))
			{
				if (GeometryCollectionComponent->GetNotifyTrailing())
				{
					if (UPhysicalMaterial* PhysicalMaterial = GeometryCollectionComponent->GetPhysicalMaterial())
					{
						if (!bApplyMaterialsFilter || IsMaterialInFilter(PhysicalMaterial->GetFName()))
						{
							Chaos::FTrailingDataExt TrailingData(InEvent);

							TrailingData.SurfaceType = PhysicalMaterial->SurfaceType;
							const FBox BoundingBox(InEvent.BoundingBox.Min(), InEvent.BoundingBox.Max());
							const FVector Extents = BoundingBox.GetSize();
							TrailingData.BoundingboxVolume = BoundingBox.GetVolume();
							TrailingData.BoundingboxExtentMin = FMath::Min3(Extents[0], Extents[1], Extents[2]);
							TrailingData.BoundingboxExtentMax = FMath::Max3(Extents[0], Extents[1], Extents[2]);

							TrailingEvents.Add(TrailingData);
						}
					}
				}
			}
		}
	}
}

void UNiagaraDataInterfaceChaosDestruction::FilterAllTrailings(TArray<Chaos::FTrailingDataExt>& AllTrailingsArray)
{
	if (SpeedToSpawnMinMax.X > 0.f ||
		SpeedToSpawnMinMax.Y > 0.f ||
		MassToSpawnMinMax.X > 0.f ||
		MassToSpawnMinMax.Y > 0.f ||
		ExtentMinToSpawnMinMax.X > 0.f ||
		ExtentMinToSpawnMinMax.Y > 0.f ||
		ExtentMaxToSpawnMinMax.X > 0.f ||
		ExtentMaxToSpawnMinMax.Y > 0.f ||
		VolumeToSpawnMinMax.X > 0.f ||
		VolumeToSpawnMinMax.Y > 0.f ||
		SolverTimeToSpawnMinMax.X > 0.f ||
		SolverTimeToSpawnMinMax.Y > 0.f ||
		SurfaceTypeToSpawn != -1 ||
		LocationXToSpawn != ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_None ||
		LocationYToSpawn != ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_None ||
		LocationZToSpawn != ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_None)
	{
		TArray<Chaos::FTrailingDataExt> FilteredAllTrailingsArray;
		FilteredAllTrailingsArray.SetNumUninitialized(AllTrailingsArray.Num());

		int32 IdxFilteredTrailings = 0;

		float MinSpeedToSpawnSquared = SpeedToSpawnMinMax.X * SpeedToSpawnMinMax.X;
		float MaxSpeedToSpawnSquared = SpeedToSpawnMinMax.Y * SpeedToSpawnMinMax.Y;

		for (int32 IdxTrailing = 0; IdxTrailing < AllTrailingsArray.Num(); ++IdxTrailing)
		{
			float TrailingSpeedSquared = AllTrailingsArray[IdxTrailing].Velocity.SizeSquared();

			if ((SpeedToSpawnMinMax.X > 0.f && SpeedToSpawnMinMax.Y < 0.f && TrailingSpeedSquared < MinSpeedToSpawnSquared) ||
				(SpeedToSpawnMinMax.X < 0.f && SpeedToSpawnMinMax.Y > 0.f && TrailingSpeedSquared > MaxSpeedToSpawnSquared) ||
				(SpeedToSpawnMinMax.X > 0.f && SpeedToSpawnMinMax.Y > 0.f && (TrailingSpeedSquared < MinSpeedToSpawnSquared || TrailingSpeedSquared > MaxSpeedToSpawnSquared)) ||
				(MassToSpawnMinMax.X > 0.f && MassToSpawnMinMax.Y < 0.f && AllTrailingsArray[IdxTrailing].Mass < MassToSpawnMinMax.X) ||
				(MassToSpawnMinMax.X < 0.f && MassToSpawnMinMax.Y > 0.f && AllTrailingsArray[IdxTrailing].Mass > MassToSpawnMinMax.Y) ||
				(MassToSpawnMinMax.X > 0.f && MassToSpawnMinMax.Y > 0.f && (AllTrailingsArray[IdxTrailing].Mass < MassToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].Mass > MassToSpawnMinMax.Y)) ||
				(ExtentMinToSpawnMinMax.X > 0.f && ExtentMinToSpawnMinMax.Y < 0.f && AllTrailingsArray[IdxTrailing].BoundingboxExtentMin < ExtentMinToSpawnMinMax.X) ||
				(ExtentMinToSpawnMinMax.X < 0.f && ExtentMinToSpawnMinMax.Y > 0.f && AllTrailingsArray[IdxTrailing].BoundingboxExtentMin > ExtentMinToSpawnMinMax.Y) ||
				(ExtentMinToSpawnMinMax.X > 0.f && ExtentMinToSpawnMinMax.Y > 0.f && (AllTrailingsArray[IdxTrailing].BoundingboxExtentMin < ExtentMinToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].BoundingboxExtentMin > ExtentMinToSpawnMinMax.Y)) ||
				(ExtentMaxToSpawnMinMax.X > 0.f && ExtentMaxToSpawnMinMax.Y < 0.f && AllTrailingsArray[IdxTrailing].BoundingboxExtentMax < ExtentMaxToSpawnMinMax.X) ||
				(ExtentMaxToSpawnMinMax.X < 0.f && ExtentMaxToSpawnMinMax.Y > 0.f && AllTrailingsArray[IdxTrailing].BoundingboxExtentMax > ExtentMaxToSpawnMinMax.Y) ||
				(ExtentMaxToSpawnMinMax.X > 0.f && ExtentMaxToSpawnMinMax.Y > 0.f && (AllTrailingsArray[IdxTrailing].BoundingboxExtentMax < ExtentMaxToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].BoundingboxExtentMax > ExtentMaxToSpawnMinMax.Y)) ||
				(VolumeToSpawnMinMax.X > 0.f && VolumeToSpawnMinMax.Y < 0.f && AllTrailingsArray[IdxTrailing].BoundingboxVolume < VolumeToSpawnMinMax.X) ||
				(VolumeToSpawnMinMax.X < 0.f && VolumeToSpawnMinMax.Y > 0.f && AllTrailingsArray[IdxTrailing].BoundingboxVolume > VolumeToSpawnMinMax.Y) ||
				(VolumeToSpawnMinMax.X > 0.f && VolumeToSpawnMinMax.Y > 0.f && (AllTrailingsArray[IdxTrailing].BoundingboxVolume < VolumeToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].BoundingboxVolume > VolumeToSpawnMinMax.Y)) ||
				(SolverTimeToSpawnMinMax.X > 0.f && SolverTimeToSpawnMinMax.Y < 0.f && SolverTime < SolverTimeToSpawnMinMax.X) ||
				(SolverTimeToSpawnMinMax.X < 0.f && SolverTimeToSpawnMinMax.Y > 0.f && SolverTime > SolverTimeToSpawnMinMax.Y) ||
				(SolverTimeToSpawnMinMax.X > 0.f && SolverTimeToSpawnMinMax.Y > 0.f && (SolverTime < SolverTimeToSpawnMinMax.X || SolverTime > SolverTimeToSpawnMinMax.Y)) ||
				(SurfaceTypeToSpawn != -1 && AllTrailingsArray[IdxTrailing].SurfaceType != SurfaceTypeToSpawn) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Min && AllTrailingsArray[IdxTrailing].Location.X <= LocationXToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Max && AllTrailingsArray[IdxTrailing].Location.X >= LocationXToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_MinMax && (AllTrailingsArray[IdxTrailing].Location.X <= LocationXToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].Location.X >= LocationXToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Min && AllTrailingsArray[IdxTrailing].Location.Y <= LocationYToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Max && AllTrailingsArray[IdxTrailing].Location.Y >= LocationYToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_MinMax && (AllTrailingsArray[IdxTrailing].Location.Y <= LocationYToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].Location.Y >= LocationYToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Min && AllTrailingsArray[IdxTrailing].Location.Z <= LocationZToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Max && AllTrailingsArray[IdxTrailing].Location.Z >= LocationZToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Inclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_MinMax && (AllTrailingsArray[IdxTrailing].Location.Z <= LocationZToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].Location.Z >= LocationZToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Min && AllTrailingsArray[IdxTrailing].Location.X > LocationXToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_Max && AllTrailingsArray[IdxTrailing].Location.X < LocationXToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationXToSpawn == ELocationXToSpawnEnum::ChaosNiagara_LocationXToSpawn_MinMax && !(AllTrailingsArray[IdxTrailing].Location.X <= LocationXToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].Location.X >= LocationXToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Min && AllTrailingsArray[IdxTrailing].Location.Y > LocationYToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_Max && AllTrailingsArray[IdxTrailing].Location.Y < LocationYToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationYToSpawn == ELocationYToSpawnEnum::ChaosNiagara_LocationYToSpawn_MinMax && !(AllTrailingsArray[IdxTrailing].Location.Y <= LocationYToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].Location.Y >= LocationYToSpawnMinMax.Y)) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Min && AllTrailingsArray[IdxTrailing].Location.Z > LocationZToSpawnMinMax.X) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_Max && AllTrailingsArray[IdxTrailing].Location.Z < LocationZToSpawnMinMax.Y) ||
				(LocationFilteringMode == ELocationFilteringModeEnum::ChaosNiagara_LocationFilteringMode_Exclusive && LocationZToSpawn == ELocationZToSpawnEnum::ChaosNiagara_LocationZToSpawn_MinMax && !(AllTrailingsArray[IdxTrailing].Location.Z <= LocationZToSpawnMinMax.X || AllTrailingsArray[IdxTrailing].Location.Z >= LocationZToSpawnMinMax.Y)))
			{
				continue;
			}

			FilteredAllTrailingsArray[IdxFilteredTrailings] = AllTrailingsArray[IdxTrailing];

			IdxFilteredTrailings++;
		}
		FilteredAllTrailingsArray.SetNum(IdxFilteredTrailings);

		// If Trailings were filtered copy FilteredAllTrailingsArray back into AllTrailings
		if (FilteredAllTrailingsArray.Num() != AllTrailingsArray.Num())
		{
			AllTrailingsArray.SetNumUninitialized(FilteredAllTrailingsArray.Num());
		}
		for (int32 Idx = 0; Idx < FilteredAllTrailingsArray.Num(); ++Idx)
		{
			AllTrailingsArray[Idx] = FilteredAllTrailingsArray[Idx];
		}

		INC_DWORD_STAT_BY(STAT_NiagaraNumFilteredAllTrailings, FilteredAllTrailingsArray.Num());
	}
}

void UNiagaraDataInterfaceChaosDestruction::SortTrailings(TArray<Chaos::FTrailingDataExt>& TrailingsArray)
{
	SCOPE_CYCLE_COUNTER(STAT_TrailingCallbackSorting);

	if (DataSortingType == EDataSortTypeEnum::ChaosNiagara_DataSortType_SortByMassMaxToMin)
	{
		TrailingsArray.Sort(UNiagaraDataInterfaceChaosDestruction::TrailingDataSortByMassPredicateMaxToMin);
	}
	else if (DataSortingType == EDataSortTypeEnum::ChaosNiagara_DataSortType_SortByMassMinToMax)
	{
		TrailingsArray.Sort(UNiagaraDataInterfaceChaosDestruction::TrailingDataSortByMassPredicateMinToMax);
	}
	else if (DataSortingType == EDataSortTypeEnum::ChaosNiagara_DataSortType_RandomShuffle)
	{
		TrailingsArray.Sort(UNiagaraDataInterfaceChaosDestruction::TrailingDataRandomShuffleSortPredicate);
	}
}

void UNiagaraDataInterfaceChaosDestruction::GetTrailingsToSpawnFromTrailings(TArray<Chaos::FTrailingDataExt>& AllTrailingsArray,
																			 TArray<Chaos::FTrailingDataExt>& TrailingsToSpawnArray)
{
	if (AllTrailingsArray.Num() <= MaxNumberOfDataEntriesToSpawn)
	{
		TrailingsToSpawnArray.SetNumUninitialized(AllTrailingsArray.Num());
		for (int32 IdxTrailing = 0; IdxTrailing < AllTrailingsArray.Num(); ++IdxTrailing)
		{
			TrailingsToSpawnArray[IdxTrailing] = AllTrailingsArray[IdxTrailing];
		}
	}
	else
	{
		float FInc = (float)AllTrailingsArray.Num() / (float)MaxNumberOfDataEntriesToSpawn;

		TrailingsToSpawnArray.SetNumUninitialized(MaxNumberOfDataEntriesToSpawn);
		for (int32 IdxTrailing = 0; IdxTrailing < MaxNumberOfDataEntriesToSpawn; ++IdxTrailing)
		{
			int32 NewIdx = FMath::FloorToInt((float)IdxTrailing * FInc);
			TrailingsToSpawnArray[IdxTrailing] = AllTrailingsArray[NewIdx];
		}
	}

	INC_DWORD_STAT_BY(STAT_NiagaraNumTrailingsToSpawnParticles, TrailingsToSpawnArray.Num());
}

int32 UNiagaraDataInterfaceChaosDestruction::SpawnParticlesFromTrailing(FSolverData SolverData,
																		Chaos::FTrailingDataExt& Trailing,
																		FNDIChaosDestruction_InstanceData* InstData,
																		float TimeData_MapsCreated,
																		int32 IdxSolver)
{
	float Latency = SolverData.Solver->GetSolverTime() - TimeData_MapsCreated;
	if (Latency <= MaxLatency)
	{
		int32 NumParticles = FMath::RandRange((int)SpawnMultiplierMinMax.X, (int)FMath::Max(SpawnMultiplierMinMax.X, SpawnMultiplierMinMax.Y));
		for (int32 Idx = 0; Idx < NumParticles; ++Idx)
		{
			if (FMath::FRand() > FMath::Clamp(SpawnChance, 0.f, 1.f))
			{
				continue;
			}

			// Compute random position
			float sign[2] = { -1.f, 1.f };
			float RandomX = sign[FMath::RandRange(0, 1)] * FMath::FRandRange(FMath::Min(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y), FMath::Max(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y));
			float RandomY = sign[FMath::RandRange(0, 1)] * FMath::FRandRange(FMath::Min(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y), FMath::Max(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y));
			float RandomZ = sign[FMath::RandRange(0, 1)] * FMath::FRandRange(FMath::Min(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y), FMath::Max(RandomPositionMagnitudeMinMax.X, RandomPositionMagnitudeMinMax.Y));
			FVector RandomPosition(RandomX, RandomY, RandomZ);

			// Compute velocity offset
			FVector VelocityOffset;
			VelocityOffset.X = FMath::FRandRange(FMath::Min(VelocityOffsetMin.X, VelocityOffsetMax.X), FMath::Max(VelocityOffsetMin.X, VelocityOffsetMax.X));
			VelocityOffset.Y = FMath::FRandRange(FMath::Min(VelocityOffsetMin.Y, VelocityOffsetMax.Y), FMath::Max(VelocityOffsetMin.Y, VelocityOffsetMax.Y));
			VelocityOffset.Z = FMath::FRandRange(FMath::Min(VelocityOffsetMin.Z, VelocityOffsetMax.Z), FMath::Max(VelocityOffsetMin.Z, VelocityOffsetMax.Z));

			// Compute random velocity using the selected Velocity Model
			FVector ParticleVelocity;
			if (RandomVelocityGenerationType == ERandomVelocityGenerationTypeEnum::ChaosNiagara_RandomVelocityGenerationType_RandomDistribution)
			{
				FVector RandomVector(FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f));
				RandomVector.Normalize();

				ParticleVelocity = RandomVector * FMath::FRandRange(RandomVelocityMagnitudeMinMax.X, FMath::Max(RandomVelocityMagnitudeMinMax.X, RandomVelocityMagnitudeMinMax.Y));
			}
			else if (RandomVelocityGenerationType == ERandomVelocityGenerationTypeEnum::ChaosNiagara_RandomVelocityGenerationType_RandomDistributionWithStreamers)
			{
				FVector RandomVector(FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f), FMath::FRandRange(-1.f, 1.f));
				RandomVector.Normalize();

				ParticleVelocity = RandomVector * FMath::FRandRange(RandomVelocityMagnitudeMinMax.X, FMath::Max(RandomVelocityMagnitudeMinMax.X, RandomVelocityMagnitudeMinMax.Y));

				float RandomMultiplier = (FMath::FRand() < 0.2) ? 1.25f : 1.f;
				ParticleVelocity *= RandomMultiplier;
			}

			// Combine all velocity components
			FVector ComputedVelocity = Trailing.Velocity * InheritedVelocityMultiplier + ParticleVelocity + VelocityOffset;

			// Clamp velocity
			if (FinalVelocityMagnitudeMinMax.X >= 0 && FinalVelocityMagnitudeMinMax.Y < 0)
			{
				if (ComputedVelocity.SizeSquared() < FinalVelocityMagnitudeMinMax.X * FinalVelocityMagnitudeMinMax.X)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.X;
				}
			}
			else if (FinalVelocityMagnitudeMinMax.X < 0 && FinalVelocityMagnitudeMinMax.Y >= 0)
			{
				if (ComputedVelocity.SizeSquared() > FinalVelocityMagnitudeMinMax.Y * FinalVelocityMagnitudeMinMax.Y)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.Y;
				}
			}
			else if (FinalVelocityMagnitudeMinMax.X >= 0 && FinalVelocityMagnitudeMinMax.Y >= 0)
			{
				if (ComputedVelocity.SizeSquared() < FinalVelocityMagnitudeMinMax.X * FinalVelocityMagnitudeMinMax.X)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.X;
				}
				else if (ComputedVelocity.SizeSquared() > FinalVelocityMagnitudeMinMax.Y * FinalVelocityMagnitudeMinMax.Y)
				{
					ComputedVelocity.Normalize();
					ComputedVelocity *= FinalVelocityMagnitudeMinMax.Y;
				}
			}

			// Debug
			FVector ParticleColor = FVector::OneVector;
			if (DebugType == EDebugTypeEnum::ChaosNiagara_DebugType_ColorBySolver)
			{
				ParticleColor = ColorArray[IdxSolver % ColorArray.Num()];
			}
			else if (DebugType == EDebugTypeEnum::ChaosNiagara_DebugType_ColorByParticleIndex)
			{
				//ParticleColor = ColorArray[Trailing.ParticleIndex % ColorArray.Num()]; #todo: ParticleIndex no longer exists
			}

			// Store principal data
			InstData->PositionArray.Add(Trailing.Location + RandomPosition);
			InstData->VelocityArray.Add(ComputedVelocity);
			InstData->SolverIDArray.Add(IdxSolver);

			// Store Trailing data
			InstData->IncomingLocationArray.Add(Trailing.Location);
			InstData->IncomingVelocity1Array.Add(Trailing.Velocity);
			InstData->IncomingAngularVelocity1Array.Add(Trailing.AngularVelocity);
			InstData->IncomingMass1Array.Add(Trailing.Mass);
			InstData->IncomingTimeArray.Add(TimeData_MapsCreated);

			// Store GeometryCollection data
			InstData->ExtentMinArray.Add(Trailing.BoundingboxExtentMin);
			InstData->ExtentMaxArray.Add(Trailing.BoundingboxExtentMax);
			InstData->VolumeArray.Add(Trailing.BoundingboxVolume);

			// Set not related to default
			InstData->IncomingAccumulatedImpulseArray.Add(FVector(ForceInitToZero));
			InstData->IncomingNormalArray.Add(FVector(ForceInitToZero));
			InstData->IncomingVelocity2Array.Add(FVector(ForceInitToZero));
			InstData->IncomingAngularVelocity2Array.Add(FVector(ForceInitToZero));
			InstData->IncomingMass2Array.Add(0.f);
		}

		return NumParticles;
	}

	return 0;
}

bool UNiagaraDataInterfaceChaosDestruction::TrailingCallback(FNDIChaosDestruction_InstanceData* InstData)
{
	int32 IdxSolver = 0;
	for (FSolverData SolverData : Solvers)
	{
		if (SolverData.Solver->GetEventFilters()->IsTrailingEventEnabled() && TrailingEvents.Num() > 0 && SolverData.Solver->GetSolverTime() > 0.f && MaxNumberOfDataEntriesToSpawn > 0)
		{
			TArray<Chaos::FTrailingDataExt>& AllTrailingsArray = TrailingEvents;
			float TimeData_MapsCreated = SolverData.Solver->GetSolverTime();

			{
				size_t SizeOfAllTrailings = sizeof(Chaos::FTrailingData) * AllTrailingsArray.Num();
				size_t SizeOfAllTrailingsIndicesByPhysicsProxy = 0;
				SET_MEMORY_STAT(STAT_AllTrailingsDataMemory, SizeOfAllTrailings);
				SET_MEMORY_STAT(STAT_AllTrailingsIndicesByPhysicsProxyMemory, SizeOfAllTrailingsIndicesByPhysicsProxy);
			}

			INC_DWORD_STAT_BY(STAT_NiagaraNumAllTrailings, AllTrailingsArray.Num());

			if (AllTrailingsArray.Num() > 0)
			{
				// Filter AllTrailings
				// In case of filtering, AllTrailings will be resized and filtered data will be copied back to AllTrailings
				FilterAllTrailings(AllTrailingsArray);

				// Sort AllCollisisons
				SortTrailings(AllTrailingsArray);

				// Get the Trailings which will spawn particles
				TArray<Chaos::FTrailingDataExt> TrailingsToSpawnArray;

				GetTrailingsToSpawnFromTrailings(AllTrailingsArray, TrailingsToSpawnArray);

				// Spawn particles for Trailings in TrailingsToSpawnArray
				for (int32 IdxTrailing = 0; IdxTrailing < TrailingsToSpawnArray.Num(); ++IdxTrailing)
				{
					int32 NumParticlesSpawned = SpawnParticlesFromTrailing(SolverData,
																		   TrailingsToSpawnArray[IdxTrailing],
																		   InstData,
																		   TimeData_MapsCreated,
																		   IdxSolver);

					if (NumParticlesSpawned > 0)
					{
						// Get/Store Geometry/Physical Material data
						FLinearColor Color(1.f, 1.f, 1.f, 1.f);
						float Friction = 0.7f, Restitution = 0.3f, Density = 1.f;
						// #GM: Disable this for now for perf
						/*
						GetMesPhysicalData(SolverData,
										   TrailingsToSpawnArray[IdxTrailing].ParticleIndexMesh == INDEX_NONE ? TrailingsToSpawnArray[IdxTrailing].ParticleIndex : TrailingsToSpawnArray[IdxTrailing].ParticleIndexMesh,
										   PhysicsProxyReverseMappingArray,
										   ParticleIndexReverseMappingArray,
										   Color,
										   Friction,
										   Restitution,
										   Density);
						*/
						for (int32 Idx = 0; Idx < NumParticlesSpawned; ++Idx)
						{
							InstData->SurfaceTypeArray.Add(TrailingsToSpawnArray[IdxTrailing].SurfaceType);
							InstData->ColorArray.Add(Color);
							InstData->FrictionArray.Add(Friction);
							InstData->RestitutionArray.Add(Restitution);
							InstData->DensityArray.Add(Density);
						}
					}
				}
			}
		}

		IdxSolver++;
	}

	INC_DWORD_STAT_BY(STAT_NiagaraNumParticlesSpawnedFromTrailings, InstData->PositionArray.Num());

	return false;
}

void UNiagaraDataInterfaceChaosDestruction::ResetInstData(FNDIChaosDestruction_InstanceData* InstData)
{
	InstData->PositionArray.Reset();
	InstData->VelocityArray.Reset();
	InstData->ExtentMinArray.Reset();
	InstData->ExtentMaxArray.Reset();
	InstData->VolumeArray.Reset();
	InstData->SolverIDArray.Reset();
	InstData->DensityArray.Reset();
	InstData->FrictionArray.Reset();
	InstData->RestitutionArray.Reset();
	InstData->SurfaceTypeArray.Reset();
	InstData->ColorArray.Reset();

	InstData->IncomingLocationArray.Reset();
	InstData->IncomingAccumulatedImpulseArray.Reset();
	InstData->IncomingNormalArray.Reset();
	InstData->IncomingVelocity1Array.Reset();
	InstData->IncomingVelocity2Array.Reset();
	InstData->IncomingAngularVelocity1Array.Reset();
	InstData->IncomingAngularVelocity2Array.Reset();
	InstData->IncomingMass1Array.Reset();
	InstData->IncomingMass2Array.Reset();
	InstData->IncomingTimeArray.Reset();

	InstData->TransformTranslationArray.Reset();
	InstData->TransformRotationArray.Reset();
	InstData->TransformScaleArray.Reset();
	InstData->BoundsArray.Reset();
}

bool UNiagaraDataInterfaceChaosDestruction::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	check(SystemInstance);
	FNDIChaosDestruction_InstanceData* InstData = (FNDIChaosDestruction_InstanceData*)PerInstanceData;

	// Update SolverTime
	if (Solvers.Num() > 0)
	{
		FSolverData SolverData = Solvers[0];
		SolverTime = SolverData.Solver->GetSolverTime();
	}

	ShouldSpawn = false;
	if (SolverTime != LastSpawnTime && SolverTime - LastSpawnTime >= 1.0 / (float)DataProcessFrequency)
	{
		// We skip the tick if we receive duplicate data from Chaos. This happens if Niagara's tick rate is faster 
		// than the chaos solver's tick rate. 

		// The first time around PrevLastSpawnedPointID and LastSpawnedPointID are both -1, and InstData is empty
		// so these assignment do not change anything.
		LastSpawnTime = SolverTime;
		LastSpawnedPointID += InstData->PositionArray.Num();
		//UE_LOGF(LogScript, Warning, "Tick, PrevLastSpawnedPointID = %d, LastSpawnedPointID = %d", PrevLastSpawnedPointID, LastSpawnedPointID);
		ShouldSpawn = true;
	} 

	ResetInstData(InstData);

	if (ShouldSpawn && DoSpawn) 
	{
		if (DataSourceType == EDataSourceTypeEnum::ChaosNiagara_DataSourceType_Collision)
		{
			SCOPE_CYCLE_COUNTER(STAT_CollisionCallback);
			return CollisionCallback(InstData);
		}
		else if (DataSourceType == EDataSourceTypeEnum::ChaosNiagara_DataSourceType_Breaking)
		{
			SCOPE_CYCLE_COUNTER(STAT_NiagaraBreakingCallback);
			return BreakingCallback(InstData);
		}
		else if (DataSourceType == EDataSourceTypeEnum::ChaosNiagara_DataSourceType_Trailing)
		{
			SCOPE_CYCLE_COUNTER(STAT_NiagaraTrailingCallback);
			return TrailingCallback(InstData);
		}
	}

	return false;
}

#if WITH_EDITORONLY_DATA
// Returns the signature of all the functions available in the data interface
void UNiagaraDataInterfaceChaosDestruction::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIChaosDestructionLocal;
	{
		// GetPosition
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPositionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Position")));			// Vector3 Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetPosition", "Helper function returning the position value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetNormal
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetNormalName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Normal")));			// Vector3 Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetNormal", "Helper function returning the normal value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetVelocity
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVelocityName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Velocity")));			// Vector3 Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetVelocity", "Helper function returning the velocity value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetAngularVelocity
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetAngularVelocityName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("AngularVelocity")));	// Vector3 Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetAngularVelocity", "Helper function returning the angular velocity value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetExtentMin
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetExtentMinName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("ExtentMin")));		// Float Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetExtentMin", "Helper function returning the min extent value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetExtentMax
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetExtentMaxName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("ExtentMax")));		// Float Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetExtentMax", "Helper function returning the max extent value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetVolume
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetVolumeName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Volume")));		// Float Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetVolume", "Helper function returning the max extent value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetParticleIdsToSpawnAtTime
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetParticleIdsToSpawnAtTimeName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));	// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time")));		    // Time in
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MinID")));			// Int Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("MaxID")));			// Int Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));		    // Int Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetParticleIdsToSpawnAtTime", "Returns the count and IDs of the particles that should spawn for a given time value."));

		OutFunctions.Add(Sig);
	}

	{
		// GetPointType
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPointTypeName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Type")));				// Int Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetPointType", "Helper function returning the type value for a given particle when spawned.\n"));

		OutFunctions.Add(Sig);
	}

	{
		// GetColor
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetColorName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In		
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetColorDef(), TEXT("Color")));			// Color Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetColor", "Helper function returning the color for a given particle when spawned."));

		OutFunctions.Add(Sig);
	}

	{
		// GetSolverTime
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSolverTimeName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("SolverTime")));		// SolverTime Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetSolverTime", "Helper function returning the SolverTime."));

		OutFunctions.Add(Sig);
	}

	{
		// GetDensity
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetDensityName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Density")));		// Float Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetDensity", "Helper function returning the max extent value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetFriction
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetFrictionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Friction")));		// Float Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetFriction", "Helper function returning the max extent value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetRestitution
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetRestitutionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Restitution")));		// Float Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetRestitution", "Helper function returning the max extent value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetSize
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSizeName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Size")));		// Int Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetSize", "Helper function returning the max extent value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetTransform
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetTransformName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Translation")));		// Int Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));		// Int Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));		// Int Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetTransform", "Helper function returning the max extent value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetSurfaceType
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetSurfaceTypeName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));		// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));			// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("SurfaceType")));		// Int Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetSurfaceType", "Helper function returning the max extent value for a given ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetCollisionData
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetCollisionDataName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));					// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));						// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionLocation")));				// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionAccumulatedImpulse")));	// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionNormal")));				// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionVelocity1")));			// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionVelocity2")));			// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionAngularVelocity1")));		// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("CollisionAngularVelocity2")));		// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CollisionMass1")));				// Float Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CollisionMass2")));				// Float Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("CollisionTime")));				// Float Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetCollisionData", "Helper function returning  ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetBreakingData
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetBreakingDataName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));					// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));						// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BreakingLocation")));				// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BreakingVelocity")));				// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("BreakingAngularVelocity")));		// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("BreakingMass")));					// Float Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("BreakingTime")));					// Float Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetBreakingData", "Helper function returning  ..."));

		OutFunctions.Add(Sig);
	}

	{
		// GetTrailingData
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetTrailingDataName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("DestructionData")));					// DestructionData in
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("ParticleID")));						// ParticleID In
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TrailingLocation")));				// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TrailingVelocity")));				// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("TrailingAngularVelocity")));		// Vector3 Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("TrailingMass")));					// Float Out
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("TrailingTime")));					// Float Out

		Sig.SetDescription(LOCTEXT("DataInterfaceChaos_GetTrailingData", "Helper function returning  ..."));

		OutFunctions.Add(Sig);
	}
}
#endif

DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetPosition);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetNormal);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetVelocity);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetAngularVelocity);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetExtentMin);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetExtentMax);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetVolume);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetParticleIdsToSpawnAtTime);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetPointType);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetColor);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetSolverTime);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetDensity);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetFriction);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetRestitution);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetTransform);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetSize);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetSurfaceType);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetCollisionData);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetBreakingData);
DEFINE_NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetTrailingData);

void UNiagaraDataInterfaceChaosDestruction::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction &OutFunc)
{
	using namespace NDIChaosDestructionLocal;

	if (BindingInfo.Name == GetPositionName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetPosition)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetNormalName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetNormal)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetVelocityName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetVelocity)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetAngularVelocityName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetAngularVelocity)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetExtentMinName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetExtentMin)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetExtentMaxName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetExtentMax)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetVolumeName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetVolume)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetParticleIdsToSpawnAtTimeName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3)
	{
		TNDIParamBinder<1, float, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetParticleIdsToSpawnAtTime)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetPointTypeName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetPointType)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetColorName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 4)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetColor)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetSolverTimeName && BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 1)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetSolverTime)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetDensityName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetDensity)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetFrictionName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetFriction)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetRestitutionName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetRestitution)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetTransformName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 10)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetTransform)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetSizeName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 3)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetSize)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetSurfaceTypeName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 1)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetSurfaceType)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetCollisionDataName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 24)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetCollisionData)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetBreakingDataName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 11)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetBreakingData)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else if (BindingInfo.Name == GetTrailingDataName && BindingInfo.GetNumInputs() == 2 && BindingInfo.GetNumOutputs() == 11)
	{
		TNDIParamBinder<1, int32, NDI_FUNC_BINDER(UNiagaraDataInterfaceChaosDestruction, GetTrailingData)>::Bind(this, BindingInfo, InstanceData, OutFunc);
	}
	else
	{
		UE_LOGF(LogTemp, Error,
			"Could not find data interface function:\n\tName: %ls\n\tInputs: %i\n\tOutputs: %i",
			*BindingInfo.Name.ToString(), BindingInfo.GetNumInputs(), BindingInfo.GetNumOutputs());
		OutFunc = FVMExternalFunction();
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetPosition(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FVector3f> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		//-TODO: LWC
		const FVector3f Sample = InstData->PositionArray.IsValidIndex(ParticleID) ? (FVector3f)InstData->PositionArray[ParticleID] : FVector3f::Zero();
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetNormal(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FVector3f> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const FVector3f Sample = InstData->IncomingNormalArray.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingNormalArray[ParticleID] : FVector3f::Zero();
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetVelocity(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FVector3f> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const FVector3f Sample = InstData->VelocityArray.IsValidIndex(ParticleID) ? (FVector3f)InstData->VelocityArray[ParticleID] : FVector3f::Zero();
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetAngularVelocity(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FVector3f> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const FVector3f Sample = InstData->IncomingAngularVelocity1Array.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingAngularVelocity1Array[ParticleID] : FVector3f::Zero();
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetExtentMin(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<float> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const float Sample = InstData->ExtentMinArray.IsValidIndex(ParticleID) ? InstData->ExtentMinArray[ParticleID] : 0.0f;
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetExtentMax(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<float> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const float Sample = InstData->ExtentMaxArray.IsValidIndex(ParticleID) ? InstData->ExtentMaxArray[ParticleID] : 0.0f;
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetVolume(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<float> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const float Sample = InstData->VolumeArray.IsValidIndex(ParticleID) ? InstData->VolumeArray[ParticleID] : 0.0f;
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename TimeParamType>
void UNiagaraDataInterfaceChaosDestruction::GetParticleIdsToSpawnAtTime(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	TimeParamType TimeParam(Context);
	FNDIOutputParam<int32> OutMinValue(Context);
	FNDIOutputParam<int32> OutMaxValue(Context);
	FNDIOutputParam<int32> OutCountValue(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		if (DoSpawn && ShouldSpawn && InstData->PositionArray.Num())
		{
			OutMinValue.SetAndAdvance(LastSpawnedPointID + 1);
			OutMaxValue.SetAndAdvance(LastSpawnedPointID + InstData->PositionArray.Num());
			OutCountValue.SetAndAdvance(InstData->PositionArray.Num());
			//UE_LOGF(LogScript, Warning, "Min = %d, Max = %d, Count = %d", LastSpawnedPointID + 1, LastSpawnedPointID + InstData->PositionArray.Num(), InstData->PositionArray.Num());
		}
		else
		{
			OutMinValue.SetAndAdvance(0);
			OutMaxValue.SetAndAdvance(0);
			OutCountValue.SetAndAdvance(0);
		}
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetPointType(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<int32> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		//if (InstData->PositionArray.Num())
		//{
		//	int32 ParticleID = ParticleIDParam.Get();
		//	// Remap ParticleID
		//	ParticleID -= LastSpawnedPointID + 1;

		//	int32 Value = 0;

		//	*OutValue.GetDest() = Value;
		//}

		//ParticleIDParam.Advance();
		//OutValue.Advance();
		OutSample.SetAndAdvance(0);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetColor(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FLinearColor> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const FLinearColor Sample = InstData->ColorArray.IsValidIndex(ParticleID) ? InstData->ColorArray[ParticleID] : FLinearColor::White;
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetSolverTime(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	FNDIOutputParam<float> OutValue(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutValue.SetAndAdvance(SolverTime);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetDensity(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<float> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const float Sample = InstData->DensityArray.IsValidIndex(ParticleID) ? InstData->DensityArray[ParticleID] : 0.0f;
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetFriction(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<float> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const float Sample = InstData->FrictionArray.IsValidIndex(ParticleID) ? InstData->FrictionArray[ParticleID] : 0.0f;
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetRestitution(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<float> OutSample(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const float Sample = InstData->RestitutionArray.IsValidIndex(ParticleID) ? InstData->RestitutionArray[ParticleID] : 0.0f;
		OutSample.SetAndAdvance(Sample);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetTransform(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FVector3f> OutTranslation(Context);
	FNDIOutputParam<FQuat4f> OutRotation(Context);
	FNDIOutputParam<FVector3f> OutScale(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		if (InstData->TransformTranslationArray.IsValidIndex(ParticleID))
		{
			//-TODO: LWC
			OutTranslation.SetAndAdvance((FVector3f)InstData->TransformTranslationArray[ParticleID]);
			OutRotation.SetAndAdvance((FQuat4f)InstData->TransformRotationArray[ParticleID]);
			OutScale.SetAndAdvance((FVector3f)InstData->TransformScaleArray[ParticleID]);
		}
		else
		{
			OutTranslation.SetAndAdvance(FVector3f::Zero());
			OutRotation.SetAndAdvance(FQuat4f::Identity);
			OutScale.SetAndAdvance(FVector3f(1.0f, 1.0f, 1.0f));
		}
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetSize(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FVector3f> OutSize(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const FVector3f Size = InstData->BoundsArray.IsValidIndex(ParticleID) ? (FVector3f)InstData->BoundsArray[ParticleID] : FVector3f::Zero();
		OutSize.SetAndAdvance(Size);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetSurfaceType(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<int32> OutValue(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);
		const int32 Value = InstData->SurfaceTypeArray.IsValidIndex(ParticleID) ? InstData->SurfaceTypeArray[ParticleID] : 0;
		OutValue.SetAndAdvance(Value);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetCollisionData(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FVector3f> OutLocation(Context);
	FNDIOutputParam<FVector3f> OutAccumulatedImpulse(Context);
	FNDIOutputParam<FVector3f> OutNormal(Context);
	FNDIOutputParam<FVector3f> OutVelocity1(Context);
	FNDIOutputParam<FVector3f> OutVelocity2(Context);
	FNDIOutputParam<FVector3f> OutAngularVelocity1(Context);
	FNDIOutputParam<FVector3f> OutAngularVelocity2(Context);
	FNDIOutputParam<float> OutMass1(Context);
	FNDIOutputParam<float> OutMass2(Context);
	FNDIOutputParam<float> OutTime(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);

		//-TODO: LWC
		OutLocation.SetAndAdvance(InstData->IncomingLocationArray.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingLocationArray[ParticleID] : FVector3f::Zero());
		OutAccumulatedImpulse.SetAndAdvance(InstData->IncomingAccumulatedImpulseArray.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingAccumulatedImpulseArray[ParticleID] : FVector3f::Zero());
		OutNormal.SetAndAdvance(InstData->IncomingNormalArray.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingNormalArray[ParticleID] : FVector3f::Zero());
		OutVelocity1.SetAndAdvance(InstData->IncomingVelocity1Array.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingVelocity1Array[ParticleID] : FVector3f::Zero());
		OutVelocity2.SetAndAdvance(InstData->IncomingVelocity2Array.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingVelocity2Array[ParticleID] : FVector3f::Zero());
		OutAngularVelocity1.SetAndAdvance(InstData->IncomingAngularVelocity1Array.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingAngularVelocity1Array[ParticleID] : FVector3f::Zero());
		OutAngularVelocity2.SetAndAdvance(InstData->IncomingAngularVelocity2Array.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingAngularVelocity2Array[ParticleID] : FVector3f::Zero());
		OutMass1.SetAndAdvance(InstData->IncomingMass1Array.IsValidIndex(ParticleID) ? InstData->IncomingMass1Array[ParticleID] : 0.0f);
		OutMass2.SetAndAdvance(InstData->IncomingMass2Array.IsValidIndex(ParticleID) ? InstData->IncomingMass2Array[ParticleID] : 0.0f);
		OutTime.SetAndAdvance(InstData->IncomingTimeArray.IsValidIndex(ParticleID) ? InstData->IncomingTimeArray[ParticleID] : 0.0f);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetBreakingData(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FVector3f> OutLocation(Context);
	FNDIOutputParam<FVector3f> OutVelocity(Context);
	FNDIOutputParam<FVector3f> OutAngularVelocity(Context);
	FNDIOutputParam<float> OutMass(Context);
	FNDIOutputParam<float> OutTime(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);

		OutLocation.SetAndAdvance(InstData->IncomingLocationArray.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingLocationArray[ParticleID] : FVector3f::Zero());
		OutVelocity.SetAndAdvance(InstData->IncomingVelocity1Array.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingVelocity1Array[ParticleID] : FVector3f::Zero());
		OutAngularVelocity.SetAndAdvance(InstData->IncomingAngularVelocity1Array.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingAngularVelocity1Array[ParticleID] : FVector3f::Zero());
		OutMass.SetAndAdvance(InstData->IncomingMass1Array.IsValidIndex(ParticleID) ? InstData->IncomingMass1Array[ParticleID] : 0.0f);
		OutTime.SetAndAdvance(InstData->IncomingTimeArray.IsValidIndex(ParticleID) ? InstData->IncomingTimeArray[ParticleID] : 0.0f);
	}
}

template<typename ParticleIDParamType>
void UNiagaraDataInterfaceChaosDestruction::GetTrailingData(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIChaosDestruction_InstanceData> InstData(Context);
	ParticleIDParamType ParticleIDParam(Context);
	FNDIOutputParam<FVector3f> OutLocation(Context);
	FNDIOutputParam<FVector3f> OutVelocity(Context);
	FNDIOutputParam<FVector3f> OutAngularVelocity(Context);
	FNDIOutputParam<float> OutMass(Context);
	FNDIOutputParam<float> OutTime(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		const int32 ParticleID = ParticleIDParam.GetAndAdvance() - (LastSpawnedPointID + 1);

		OutLocation.SetAndAdvance(InstData->IncomingLocationArray.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingLocationArray[ParticleID] : FVector3f::Zero());
		OutVelocity.SetAndAdvance(InstData->IncomingVelocity1Array.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingVelocity1Array[ParticleID] : FVector3f::Zero());
		OutAngularVelocity.SetAndAdvance(InstData->IncomingAngularVelocity1Array.IsValidIndex(ParticleID) ? (FVector3f)InstData->IncomingAngularVelocity1Array[ParticleID] : FVector3f::Zero());
		OutMass.SetAndAdvance(InstData->IncomingMass1Array.IsValidIndex(ParticleID) ? InstData->IncomingMass1Array[ParticleID] : 0.0f);
		OutTime.SetAndAdvance(InstData->IncomingTimeArray.IsValidIndex(ParticleID) ? InstData->IncomingTimeArray[ParticleID] : 0.0f);
	}
}

//----------------------------------------------------------------------------
// GPU sim functionality
//
#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceChaosDestruction::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIChaosDestructionLocal::TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<NDIChaosDestructionLocal::FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceChaosDestruction::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs = { {TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol}, };
	AppendTemplateHLSL(OutHLSL, NDIChaosDestructionLocal::TemplateShaderFilePath, TemplateArgs);
}

bool UNiagaraDataInterfaceChaosDestruction::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIChaosDestructionLocal;

	static const TSet<FName> ValidGpuFunction =
	{
		GetPositionName,
		GetNormalName,
		GetVelocityName,
		GetAngularVelocityName,
		GetExtentMinName,
		GetExtentMaxName,
		GetVolumeName,
		GetParticleIdsToSpawnAtTimeName,
		GetPointTypeName,
		GetColorName,
		GetSolverTimeName,
		GetDensityName,
		GetFrictionName,
		GetRestitutionName,
		GetTransformName,
		GetSizeName,
		GetSurfaceTypeName,
		GetCollisionDataName,
		GetBreakingDataName,
		GetTrailingDataName,
	};

	return ValidGpuFunction.Contains(FunctionInfo.DefinitionName);
}
#endif

void UNiagaraDataInterfaceChaosDestruction::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIChaosDestructionLocal::FShaderParameters>();
}

void UNiagaraDataInterfaceChaosDestruction::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNiagaraDataInterfaceProxyChaosDestruction& DIProxy = Context.GetProxy<FNiagaraDataInterfaceProxyChaosDestruction>();
	FNiagaraDIChaosDestruction_GPUData* InstanceData = DIProxy.SystemsToGPUInstanceData.Find(Context.GetSystemInstanceID());
	NDIChaosDestructionLocal::FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<NDIChaosDestructionLocal::FShaderParameters>();

	if (InstanceData != nullptr && (InstanceData->PositionArray.Num() > 0))
	{
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->PositionBuffer, InstanceData->RDGPositionSRV, InstanceData->PositionArray, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_Position"));
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->VelocityBuffer, InstanceData->RDGVelocitySRV, InstanceData->VelocityArray, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_Velocity"));
		NDIChaosDestructionLocal::QueueBufferUpload<float>(Context, ShaderParameters->ExtentMinBuffer, InstanceData->RDGExtentMinBufferSRV, InstanceData->ExtentMinArray, 0.0f, PF_R32_FLOAT, TEXT("ChaosGPUBuffer_ExtentMin"));
		NDIChaosDestructionLocal::QueueBufferUpload<float>(Context, ShaderParameters->ExtentMaxBuffer, InstanceData->RDGExtentMaxBufferSRV, InstanceData->ExtentMaxArray, 0.0f, PF_R32_FLOAT, TEXT("ChaosGPUBuffer_ExtentMax"));
		NDIChaosDestructionLocal::QueueBufferUpload<float>(Context, ShaderParameters->VolumeBuffer, InstanceData->RDGVolumeBufferSRV, InstanceData->VolumeArray, 0.0f, PF_R32_FLOAT, TEXT("ChaosGPUBuffer_Volume"));
		NDIChaosDestructionLocal::QueueBufferUpload<int>(Context, ShaderParameters->SolverIDBuffer, InstanceData->RDGSolverIDBufferSRV, InstanceData->SolverIDArray, 0, PF_R32_SINT, TEXT("ChaosGPUBuffer_SolverID"));
		NDIChaosDestructionLocal::QueueBufferUpload<float>(Context, ShaderParameters->DensityBuffer, InstanceData->RDGDensityBufferSRV, InstanceData->DensityArray, 0.0f, PF_R32_FLOAT, TEXT("ChaosGPUBuffer_Density"));
		NDIChaosDestructionLocal::QueueBufferUpload<float>(Context, ShaderParameters->FrictionBuffer, InstanceData->RDGFrictionBufferSRV, InstanceData->FrictionArray, 0.0f, PF_R32_FLOAT, TEXT("ChaosGPUBuffer_Friction"));
		NDIChaosDestructionLocal::QueueBufferUpload<float>(Context, ShaderParameters->RestitutionBuffer, InstanceData->RDGRestitutionBufferSRV, InstanceData->RestitutionArray, 0.0f, PF_R32_FLOAT, TEXT("ChaosGPUBuffer_Restitution"));
		NDIChaosDestructionLocal::QueueBufferUpload<int>(Context, ShaderParameters->SurfaceTypeBuffer, InstanceData->RDGSurfaceTypeBufferSRV, InstanceData->SurfaceTypeArray, 0, PF_R32_SINT, TEXT("ChaosGPUBuffer_SurfaceType"));
		NDIChaosDestructionLocal::QueueBufferUpload<FLinearColor>(Context, ShaderParameters->ColorBuffer, InstanceData->RDGColorBufferSRV, InstanceData->ColorArray, FLinearColor::Black, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_Color"));

		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->IncomingLocationBuffer, InstanceData->RDGIncomingLocationBufferSRV, InstanceData->IncomingLocationArray, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_IncomingLocation"));
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->IncomingAccumulatedImpulseBuffer, InstanceData->RDGIncomingAccumulatedImpulseBufferSRV, InstanceData->IncomingAccumulatedImpulseArray, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_IncomingAccumulatedImpulse"));
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->IncomingNormalBuffer, InstanceData->RDGIncomingNormalBufferSRV, InstanceData->IncomingNormalArray, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_IncomingNormal"));
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->IncomingVelocity1Buffer, InstanceData->RDGIncomingVelocity1BufferSRV, InstanceData->IncomingVelocity1Array, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_IncomingVelocity1"));
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->IncomingVelocity2Buffer, InstanceData->RDGIncomingVelocity2BufferSRV, InstanceData->IncomingVelocity2Array, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_IncomingVelocity2"));
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->IncomingAngularVelocity1Buffer, InstanceData->RDGIncomingAngularVelocity1BufferSRV, InstanceData->IncomingAngularVelocity1Array, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_IncomingAngualrVelocity1"));
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->IncomingAngularVelocity2Buffer, InstanceData->RDGIncomingAngularVelocity2BufferSRV, InstanceData->IncomingAngularVelocity2Array, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_IncomingAngualrVelocity2"));
		NDIChaosDestructionLocal::QueueBufferUpload<float>(Context, ShaderParameters->IncomingMass1Buffer, InstanceData->RDGIncomingMass1BufferSRV, InstanceData->IncomingMass1Array, 0.0f, PF_R32_FLOAT, TEXT("ChaosGPUBuffer_IncomingMass1"));
		NDIChaosDestructionLocal::QueueBufferUpload<float>(Context, ShaderParameters->IncomingMass2Buffer, InstanceData->RDGIncomingMass2BufferSRV, InstanceData->IncomingMass2Array, 0.0f, PF_R32_FLOAT, TEXT("ChaosGPUBuffer_IncomingMass2"));
		NDIChaosDestructionLocal::QueueBufferUpload<float>(Context, ShaderParameters->IncomingTimeBuffer, InstanceData->RDGIncomingTimeBufferSRV, InstanceData->IncomingTimeArray, 0.0f, PF_R32_FLOAT, TEXT("ChaosGPUBuffer_IncomingTime"));

		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->TransformTranslationBuffer, InstanceData->RDGTransformTranslationBufferSRV, InstanceData->TransformTranslationArray, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_TransformTranslation"));
		NDIChaosDestructionLocal::QueueBufferUpload<FQuat4f>(Context, ShaderParameters->TransformRotationBuffer, InstanceData->RDGTransformRotationBufferSRV, InstanceData->TransformRotationArray, FQuat4f::Identity, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_TransformRotation"));
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->TransformScaleBuffer, InstanceData->RDGTransformScaleBufferSRV, InstanceData->TransformScaleArray, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_TransformScale"));
		NDIChaosDestructionLocal::QueueBufferUpload<FVector3f>(Context, ShaderParameters->BoundsBuffer, InstanceData->RDGBoundsBufferSRV, InstanceData->BoundsArray, FVector3f::ZeroVector, PF_A32B32G32R32F, TEXT("ChaosGPUBuffer_TransformBounds"));

		ShaderParameters->LastSpawnedPointID = InstanceData->LastSpawnedPointID;
		ShaderParameters->SolverTime = InstanceData->SolverTime;
	}
	else
	{
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->PositionBuffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->VelocityBuffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->ExtentMinBuffer, PF_R32_FLOAT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->ExtentMaxBuffer, PF_R32_FLOAT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->VolumeBuffer, PF_R32_FLOAT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->SolverIDBuffer, PF_R32_SINT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->DensityBuffer, PF_R32_FLOAT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->FrictionBuffer, PF_R32_FLOAT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->RestitutionBuffer, PF_R32_FLOAT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->SurfaceTypeBuffer, PF_R32_SINT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->ColorBuffer, PF_A32B32G32R32F);

		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingLocationBuffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingAccumulatedImpulseBuffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingNormalBuffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingVelocity1Buffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingVelocity2Buffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingAngularVelocity1Buffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingAngularVelocity2Buffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingMass1Buffer, PF_R32_FLOAT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingMass2Buffer, PF_R32_FLOAT);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->IncomingTimeBuffer, PF_R32_FLOAT);

		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->TransformTranslationBuffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->TransformRotationBuffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->TransformScaleBuffer, PF_A32B32G32R32F);
		NDIChaosDestructionLocal::GetEmptyBuffer(Context, ShaderParameters->BoundsBuffer, PF_A32B32G32R32F);

		ShaderParameters->LastSpawnedPointID = 0;
		ShaderParameters->SolverTime = 0.0f;
	}
}

void UNiagaraDataInterfaceChaosDestruction::PushToRenderThreadImpl()
{
	check(Proxy);
	FNiagaraDataInterfaceProxyChaosDestruction* RT_Proxy = GetProxyAs<FNiagaraDataInterfaceProxyChaosDestruction>();

	int32 RT_LastSpawnedPointID = LastSpawnedPointID;
	float RT_SolverTime = SolverTime;

	ENQUEUE_RENDER_COMMAND(FPushDIChaosDestructionToRT) (
		[RT_Proxy, RT_LastSpawnedPointID, RT_SolverTime](FRHICommandListImmediate& RHICmdList)
		{
			RT_Proxy->LastSpawnedPointID = RT_LastSpawnedPointID;
			RT_Proxy->SolverTime = RT_SolverTime;
		}
	);
}

void UNiagaraDataInterfaceChaosDestruction::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	check(Proxy);

	FNiagaraDIChaosDestruction_InstanceDataToPassToRT* DataToPass = new (DataForRenderThread) FNiagaraDIChaosDestruction_InstanceDataToPassToRT;
	FNDIChaosDestruction_InstanceData* InstanceData = static_cast<FNDIChaosDestruction_InstanceData*>(PerInstanceData);
	check(InstanceData);

	DataToPass->SolverTime = GetSolverTime();
	DataToPass->LastSpawnedPointID = GetLastSpawnedPointID();

	DataToPass->PositionArray = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->PositionArray);
	DataToPass->VelocityArray = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->VelocityArray);
	DataToPass->ExtentMinArray = InstanceData->ExtentMinArray;
	DataToPass->ExtentMaxArray = InstanceData->ExtentMaxArray;
	DataToPass->VolumeArray = InstanceData->VolumeArray;
	DataToPass->SolverIDArray = InstanceData->SolverIDArray;
	DataToPass->DensityArray = InstanceData->DensityArray;
	DataToPass->FrictionArray = InstanceData->FrictionArray;
	DataToPass->RestitutionArray = InstanceData->RestitutionArray;
	DataToPass->TransformTranslationArray = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->TransformTranslationArray);
	DataToPass->TransformRotationArray = NDIChaosDestructionLocal::CopyQuatArray(InstanceData->TransformRotationArray);
	DataToPass->TransformScaleArray = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->TransformScaleArray);
	DataToPass->BoundsArray = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->BoundsArray);
	DataToPass->SurfaceTypeArray = InstanceData->SurfaceTypeArray;
	DataToPass->ColorArray = InstanceData->ColorArray;
	DataToPass->IncomingLocationArray = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->IncomingLocationArray);
	DataToPass->IncomingAccumulatedImpulseArray = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->IncomingAccumulatedImpulseArray);
	DataToPass->IncomingNormalArray = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->IncomingNormalArray);
	DataToPass->IncomingVelocity1Array = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->IncomingVelocity1Array);
	DataToPass->IncomingVelocity2Array = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->IncomingVelocity2Array);
	DataToPass->IncomingAngularVelocity1Array = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->IncomingAngularVelocity1Array);
	DataToPass->IncomingAngularVelocity2Array = NDIChaosDestructionLocal::CopyVectorArray(InstanceData->IncomingAngularVelocity2Array);
	DataToPass->IncomingMass1Array = InstanceData->IncomingMass1Array;
	DataToPass->IncomingMass2Array = InstanceData->IncomingMass2Array;
	DataToPass->IncomingTimeArray = InstanceData->IncomingTimeArray;
}

void FNiagaraDataInterfaceProxyChaosDestruction::CreatePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());
	check(!SystemsToGPUInstanceData.Contains(SystemInstance));
	SystemsToGPUInstanceData.Add(SystemInstance, FNiagaraDIChaosDestruction_GPUData());
}

void FNiagaraDataInterfaceProxyChaosDestruction::DestroyInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());
	check(SystemsToGPUInstanceData.Contains(SystemInstance));
	SystemsToGPUInstanceData.Remove(SystemInstance);
}

void FNiagaraDataInterfaceProxyChaosDestruction::ConsumePerInstanceDataFromGameThread(void* PerInstanceDataFromGameThread, const FNiagaraSystemInstanceID& Instance)
{ 
	FNiagaraDIChaosDestruction_InstanceDataToPassToRT* InstanceData = static_cast<FNiagaraDIChaosDestruction_InstanceDataToPassToRT*>(PerInstanceDataFromGameThread);
	FNiagaraDIChaosDestruction_GPUData* DataPtr = SystemsToGPUInstanceData.Find(Instance);
	if (ensure(DataPtr))
	{
		FNiagaraDIChaosDestruction_GPUData& Data = *DataPtr;
	
		Data.ResetAll();

		Data.SolverTime = InstanceData->SolverTime;
		Data.LastSpawnedPointID = InstanceData->LastSpawnedPointID;

		Data.PositionArray = MoveTemp(InstanceData->PositionArray);
		Data.VelocityArray = MoveTemp(InstanceData->VelocityArray);
		Data.ExtentMinArray = MoveTemp(InstanceData->ExtentMinArray);
		Data.ExtentMaxArray = MoveTemp(InstanceData->ExtentMaxArray);
		Data.VolumeArray = MoveTemp(InstanceData->VolumeArray);
		Data.SolverIDArray = MoveTemp(InstanceData->SolverIDArray);
		Data.DensityArray = MoveTemp(InstanceData->DensityArray);
		Data.FrictionArray = MoveTemp(InstanceData->FrictionArray);
		Data.RestitutionArray = MoveTemp(InstanceData->RestitutionArray);
		Data.TransformTranslationArray = MoveTemp(InstanceData->TransformTranslationArray);
		Data.TransformRotationArray = MoveTemp(InstanceData->TransformRotationArray);
		Data.TransformScaleArray = MoveTemp(InstanceData->TransformScaleArray);
		Data.BoundsArray = MoveTemp(InstanceData->BoundsArray);
		Data.SurfaceTypeArray = MoveTemp(InstanceData->SurfaceTypeArray);
		Data.ColorArray = MoveTemp(InstanceData->ColorArray);
		Data.IncomingLocationArray = MoveTemp(InstanceData->IncomingLocationArray);
		Data.IncomingAccumulatedImpulseArray = MoveTemp(InstanceData->IncomingAccumulatedImpulseArray);
		Data.IncomingNormalArray = MoveTemp(InstanceData->IncomingNormalArray);
		Data.IncomingVelocity1Array = MoveTemp(InstanceData->IncomingVelocity1Array);
		Data.IncomingVelocity2Array = MoveTemp(InstanceData->IncomingVelocity2Array);
		Data.IncomingAngularVelocity1Array = MoveTemp(InstanceData->IncomingAngularVelocity1Array);
		Data.IncomingAngularVelocity2Array = MoveTemp(InstanceData->IncomingAngularVelocity2Array);
		Data.IncomingMass1Array = MoveTemp(InstanceData->IncomingMass1Array);
		Data.IncomingMass2Array = MoveTemp(InstanceData->IncomingMass2Array);
		Data.IncomingTimeArray = MoveTemp(InstanceData->IncomingTimeArray);

		InstanceData->~FNiagaraDIChaosDestruction_InstanceDataToPassToRT();
	}
}

void FNiagaraDataInterfaceProxyChaosDestruction::PostSimulate(const FNDIGpuComputePostSimulateContext& Context)
{
	// Clear out the transient data we may have generated if this is the final post simulate call
	if (Context.IsFinalPostSimulate())
	{
		for (auto it = SystemsToGPUInstanceData.CreateIterator(); it; ++it)
		{
			FNiagaraDIChaosDestruction_GPUData& InstanceData = it.Value();
			InstanceData.RDGPositionSRV = nullptr;
			InstanceData.RDGVelocitySRV = nullptr;
			InstanceData.RDGExtentMinBufferSRV = nullptr;
			InstanceData.RDGExtentMaxBufferSRV = nullptr;
			InstanceData.RDGVolumeBufferSRV = nullptr;
			InstanceData.RDGSolverIDBufferSRV = nullptr;
			InstanceData.RDGDensityBufferSRV = nullptr;
			InstanceData.RDGFrictionBufferSRV = nullptr;
			InstanceData.RDGRestitutionBufferSRV = nullptr;
			InstanceData.RDGSurfaceTypeBufferSRV = nullptr;
			InstanceData.RDGColorBufferSRV = nullptr;
			InstanceData.RDGIncomingLocationBufferSRV = nullptr;
			InstanceData.RDGIncomingAccumulatedImpulseBufferSRV = nullptr;
			InstanceData.RDGIncomingNormalBufferSRV = nullptr;
			InstanceData.RDGIncomingVelocity1BufferSRV = nullptr;
			InstanceData.RDGIncomingVelocity2BufferSRV = nullptr;
			InstanceData.RDGIncomingAngularVelocity1BufferSRV = nullptr;
			InstanceData.RDGIncomingAngularVelocity2BufferSRV = nullptr;
			InstanceData.RDGIncomingMass1BufferSRV = nullptr;
			InstanceData.RDGIncomingMass2BufferSRV = nullptr;
			InstanceData.RDGIncomingTimeBufferSRV = nullptr;
			InstanceData.RDGTransformTranslationBufferSRV = nullptr;
			InstanceData.RDGTransformRotationBufferSRV = nullptr;
			InstanceData.RDGTransformScaleBufferSRV = nullptr;
			InstanceData.RDGBoundsBufferSRV = nullptr;
		}
	}
}

#undef LOCTEXT_NAMESPACE

