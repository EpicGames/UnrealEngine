// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceGeometryCollection.h"
#include "CoreMinimal.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraRenderer.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSimStageData.h"
#include "NiagaraSystemInstance.h"
#include "RenderGraphBuilder.h"
#include "RenderingThread.h"
#include "UnifiedBuffer.h"
#include "Engine/World.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfaceGeometryCollection)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceGeometryCollection"
DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollection, Log, All);

//------------------------------------------------------------------------------------------------------------

namespace NDIGeometryCollectionLocal
{
	static const FName GetClosestPointNoNormalName(TEXT("GetClosestPointNoNormal"));
	static const FName GetNumElementsName(TEXT("GetNumElements"));
	static const FName GetElementBoundsName(TEXT("GetElementBounds"));
	static const FName GetTransformComponentName(TEXT("GetElementTransform"));
	static const FName SetTransformComponentName(TEXT("SetElementTransform"));
	static const FName SetTransformWorldName(TEXT("SetElementTransformWS"));
	static const FName GetComponentWSTransformName(TEXT("GetGeometryComponentTransform"));
	static const TCHAR* TemplateShaderFilePath = TEXT("/Plugin/Experimental/ChaosNiagara/NiagaraDataInterfaceGeometryCollection.ush");

	template<typename BufferType, EPixelFormat PixelFormat>
	void CreateInternalBuffer(FRHICommandListBase& RHICmdList, FReadBuffer& OutputBuffer, uint32 ElementCount)
	{
		if (ElementCount > 0)
		{
			OutputBuffer.Initialize(RHICmdList, TEXT("FNDIGeometryCollectionBuffer"), sizeof(BufferType), ElementCount, PixelFormat, BUF_Static);
		}
	}

	template<typename BufferType>
	void UpdateInternalBuffer(FRHICommandListBase& RHICmdList, const TArray<BufferType>& InputData, FReadBuffer& OutputBuffer)
	{
		uint32 ElementCount = InputData.Num();
		if (ElementCount > 0 && OutputBuffer.Buffer.IsValid())
		{
			const uint32 BufferBytes = sizeof(BufferType) * ElementCount;

			void* OutputData = RHICmdList.LockBuffer(OutputBuffer.Buffer, 0, BufferBytes, RLM_WriteOnly);

			FMemory::Memcpy(OutputData, InputData.GetData(), BufferBytes);
			RHICmdList.UnlockBuffer(OutputBuffer.Buffer);
		}
	}

	struct FGeometryCollectionDIFunctionVersion
	{
		enum Type
		{
			InitialVersion = 0,
			DIRefactor = 1,
			AddedElementIndexOutput = 2,
			
			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	};
}

//------------------------------------------------------------------------------------------------------------

void FNDIGeometryCollectionBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	NDIGeometryCollectionLocal::CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, WorldTransformBuffer, 3 * NumPieces);
	NDIGeometryCollectionLocal::CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, PrevWorldTransformBuffer, 3 * NumPieces);

	NDIGeometryCollectionLocal::CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, WorldInverseTransformBuffer, 3 * NumPieces);
	NDIGeometryCollectionLocal::CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, PrevWorldInverseTransformBuffer, 3 * NumPieces);

	NDIGeometryCollectionLocal::CreateInternalBuffer<FVector4f, EPixelFormat::PF_A32B32G32R32F>(RHICmdList, BoundsBuffer, NumPieces);
}

void FNDIGeometryCollectionBuffer::ReleaseRHI()
{
	WorldTransformBuffer.Release();
	PrevWorldTransformBuffer.Release();
	WorldInverseTransformBuffer.Release();
	PrevWorldInverseTransformBuffer.Release();
	BoundsBuffer.Release();
}

//------------------------------------------------------------------------------------------------------------



void FNDIGeometryCollectionData::Release()
{
	if (AssetBuffer)
	{
		ENQUEUE_RENDER_COMMAND(DeleteResource)(
			[ParamPointerToRelease = AssetBuffer](FRHICommandListImmediate& RHICmdList)
			{
				ParamPointerToRelease->ReleaseResource();
				delete ParamPointerToRelease;
			});
		AssetBuffer = nullptr;
	}
}

void FNDIGeometryCollectionData::Init(UNiagaraDataInterfaceGeometryCollection* Interface, FNiagaraSystemInstance* SystemInstance)
{
	AssetBuffer = nullptr;

	if (Interface && SystemInstance)
	{
		if (const UGeometryCollection* GeometryCollection = ResolvedSource.GetGeometryCollection())
		{			
			const TSharedPtr<FGeometryCollection> Collection = GeometryCollection->GetGeometryCollection();
			const TManagedArray<FBox>& BoundingBoxes = Collection->BoundingBox;
			const TManagedArray<int32>& TransformIndexArray = Collection->TransformIndex;

			int NumPieces = 0;
			for (int i = 0; i < BoundingBoxes.Num(); ++i)
			{
				int32 CurrTransformIndex = TransformIndexArray[i];

				if (Interface->bIncludeIntermediateBones || Collection->Children[CurrTransformIndex].Num() == 0)
				{
					NumPieces++;
				}
			}

			AssetArrays = new FNDIGeometryCollectionArrays();
			AssetArrays->Resize(NumPieces);

			AssetBuffer = new FNDIGeometryCollectionBuffer();
			AssetBuffer->SetNumPieces(NumPieces);
			BeginInitResource(AssetBuffer);

			FVector Origin(ForceInitToZero);
			FVector Extents(ForceInitToZero);
			if (ResolvedSource.Component.IsValid())
			{
				ResolvedSource.Component->Bounds.GetBox().GetCenterAndExtents(Origin, Extents);
			}

			FNiagaraLWCConverter LwcConverter = SystemInstance->GetLWCConverter();
			BoundsOrigin = LwcConverter.ConvertWorldToSimulationVector(Origin);
			BoundsExtent = static_cast<FVector3f>(Extents);

			int PieceIndex = 0;
			for (int i = 0; i < BoundingBoxes.Num(); ++i)
			{
				int32 CurrTransformIndex = TransformIndexArray[i];

				if (Interface->bIncludeIntermediateBones || Collection->Children[CurrTransformIndex].Num() == 0)
				{
					FBox CurrBox = BoundingBoxes[i];
					FVector3f BoxSize = FVector3f(CurrBox.Max - CurrBox.Min);
					AssetArrays->BoundsBuffer[PieceIndex] = FVector4f(BoxSize.X, BoxSize.Y, BoxSize.Z, 0);

					PieceIndex++;
				}
			}
		}
		else
		{
			AssetArrays = new FNDIGeometryCollectionArrays();
			AssetArrays->Resize(1);

			AssetBuffer = new FNDIGeometryCollectionBuffer();
			AssetBuffer->SetNumPieces(1);
			BeginInitResource(AssetBuffer);
		}
	}

}

void FNDIGeometryCollectionData::Update(UNiagaraDataInterfaceGeometryCollection* Interface, FNiagaraSystemInstance* SystemInstance)
{
	if (Interface && SystemInstance)
	{		
		TickingGroup = ComputeTickingGroup();
		
		if (const UGeometryCollection* GeoCollection = ResolvedSource.GetGeometryCollection())
		{
			RootTransform = ResolvedSource.GetComponentRootTransform(SystemInstance);

			const TSharedPtr<FGeometryCollection> Collection = GeoCollection->GetGeometryCollection();
			const TManagedArray<FBox>& BoundingBoxes = Collection->BoundingBox;
			const TManagedArray<int32>& TransformIndexArray = Collection->TransformIndex;

			int NumPieces = 0;
			for (int i = 0; i < BoundingBoxes.Num(); ++i)
			{
				int32 CurrTransformIndex = TransformIndexArray[i];

				if (Interface->bIncludeIntermediateBones || Collection->Children[CurrTransformIndex].Num() == 0)
				{
					NumPieces++;
				}
			}

			if (NumPieces != AssetArrays->BoundsBuffer.Num())
			{
				Init(Interface, SystemInstance);
				bNeedsRenderUpdate = true;
				AssetArrays->ComponentRestTransformBuffer = ResolvedSource.GetLocalRestTransforms();
			}
			else
			{
				TArray<FTransform> NewTransforms = ResolvedSource.GetLocalRestTransforms();
				int32 TransformCount = NewTransforms.Num();
				if (TransformCount != AssetArrays->ComponentRestTransformBuffer.Num() ||
					FMemory::Memcmp(NewTransforms.GetData(), AssetArrays->ComponentRestTransformBuffer.GetData(), TransformCount * sizeof(FTransform)) != 0)
				{
					AssetArrays->ComponentRestTransformBuffer = MoveTemp(NewTransforms);
					bNeedsRenderUpdate = true;
				}
			}
			
			FVector Origin(ForceInitToZero);
			FVector Extents(ForceInitToZero);
			if (ResolvedSource.Component.IsValid())
			{
				ResolvedSource.Component->Bounds.GetBox().GetCenterAndExtents(Origin, Extents);
			}
			else
			{
				// if the extent is 0 then some functions won't work with the preview collection, as it doesn't have a component
				Extents = FVector::OneVector * UE_FLOAT_HUGE_DISTANCE;
			}

			FNiagaraLWCConverter LwcConverter = SystemInstance->GetLWCConverter();
			BoundsOrigin = LwcConverter.ConvertWorldToSimulationVector(Origin);
			BoundsExtent = static_cast<FVector3f>(Extents);

			int PieceIndex = 0;
			for (int i = 0; i < BoundingBoxes.Num(); ++i)
			{
				int32 CurrTransformIndex = TransformIndexArray[i];
				if (Interface->bIncludeIntermediateBones || Collection->Children[CurrTransformIndex].Num() == 0)
				{
					ensureMsgf(AssetArrays->ComponentRestTransformBuffer.IsValidIndex(CurrTransformIndex), TEXT("The local rest transforms and the indices for the transform mapping should always match."));
					AssetArrays->ElementIndexToTransformBufferMapping[PieceIndex] = CurrTransformIndex;
					
					int32 TransformIndex = 3 * PieceIndex;
					AssetArrays->PrevWorldInverseTransformBuffer[TransformIndex] = AssetArrays->WorldInverseTransformBuffer[TransformIndex];
					AssetArrays->PrevWorldInverseTransformBuffer[TransformIndex + 1] = AssetArrays->WorldInverseTransformBuffer[TransformIndex + 1];
					AssetArrays->PrevWorldInverseTransformBuffer[TransformIndex + 2] = AssetArrays->WorldInverseTransformBuffer[TransformIndex + 2];

					AssetArrays->PrevWorldTransformBuffer[TransformIndex] = AssetArrays->WorldTransformBuffer[TransformIndex];
					AssetArrays->PrevWorldTransformBuffer[TransformIndex + 1] = AssetArrays->WorldTransformBuffer[TransformIndex + 1];
					AssetArrays->PrevWorldTransformBuffer[TransformIndex + 2] = AssetArrays->WorldTransformBuffer[TransformIndex + 2];

					FBox CurrBox = BoundingBoxes[i];

					// #todo(dmp): save this somewhere in an array?
					FVector LocalTranslation = (CurrBox.Max + CurrBox.Min) * .5;
					FTransform LocalOffset(LocalTranslation);

					const FTransform3f CurrTransform(LocalOffset * ResolvedSource.GetComponentSpaceTransform(CurrTransformIndex) * RootTransform);
					CurrTransform.ToMatrixWithScale().To3x4MatrixTranspose(&AssetArrays->WorldTransformBuffer[TransformIndex].X);

					const FTransform3f CurrInverse = CurrTransform.Inverse();
					CurrInverse.ToMatrixWithScale().To3x4MatrixTranspose(&AssetArrays->WorldInverseTransformBuffer[TransformIndex].X);

					PieceIndex++;
				}
			}
		}		
	}
}

ETickingGroup FNDIGeometryCollectionData::ComputeTickingGroup()
{
	TickingGroup = NiagaraFirstTickGroup;
	return TickingGroup;
}

const UGeometryCollection* FResolvedNiagaraGeometryCollection::GetGeometryCollection() const
{
	if (Collection.IsValid())
	{
		return Collection.Get();
	}
	if (Component.IsValid())
	{
		return Component->GetRestCollection();
	}
	return nullptr;
}

FTransform FResolvedNiagaraGeometryCollection::GetComponentRootTransform(FNiagaraSystemInstance* SystemInstance) const
{
	FTransform ComponentTransform = FTransform::Identity;
	if (Component.IsValid())
	{
		ComponentTransform = Component->GetComponentTransform();
	}
	ComponentTransform.AddToTranslation(FVector(SystemInstance->GetLWCTile()) * -FLargeWorldRenderScalar::GetTileSize());
	return ComponentTransform;
}

FTransform FResolvedNiagaraGeometryCollection::GetComponentSpaceTransform(int32 TransformIndex) const
{
	if (Component.IsValid())
	{
		const TArray<FTransform3f>& ComponentSpaceTransforms = Component->GetComponentSpaceTransforms3f();
		if (ComponentSpaceTransforms.IsValidIndex(TransformIndex))
		{
			return FTransform(ComponentSpaceTransforms[TransformIndex]);
		}
	}
	return FTransform();
}

TArray<FTransform> FResolvedNiagaraGeometryCollection::GetLocalRestTransforms() const
{
	if (Component.IsValid())
	{
		return Component->GetLocalRestTransforms();
	}
	if (Collection.IsValid())
	{
		TArray<FTransform> InitialLocalTransforms;
		const FGeometryCollection& RestGeometryCollection = *Collection->GetGeometryCollection();
		GeometryCollectionAlgo::GlobalMatrices(RestGeometryCollection.Transform, RestGeometryCollection.Parent, InitialLocalTransforms);
		return InitialLocalTransforms;
	}
	return TArray<FTransform>();
}

//------------------------------------------------------------------------------------------------------------

void FNDIGeometryCollectionProxy::ConsumePerInstanceDataFromGameThread(void* DataFromGameThread, const FNiagaraSystemInstanceID& Instance)
{
	check(IsInRenderingThread());

	FNDIGeometryCollectionData* SourceData = static_cast<FNDIGeometryCollectionData*>(DataFromGameThread);
	FNDIGeometryCollectionData& TargetData = SystemInstancesToProxyData.FindOrAdd(Instance);

	TargetData.AssetBuffer = SourceData->AssetBuffer;		
	TargetData.AssetArrays = SourceData->AssetArrays;
	TargetData.TickingGroup = SourceData->TickingGroup;
	TargetData.RootTransform = SourceData->RootTransform;
	TargetData.BoundsOrigin = SourceData->BoundsOrigin;
	TargetData.BoundsExtent = SourceData->BoundsExtent;
	
	SourceData->~FNDIGeometryCollectionData();
}

void FNDIGeometryCollectionProxy::InitializePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());
	check(!SystemInstancesToProxyData.Contains(SystemInstance));
	
	SystemInstancesToProxyData.Add(SystemInstance);
}

void FNDIGeometryCollectionProxy::DestroyPerInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());	

	SystemInstancesToProxyData.Remove(SystemInstance);
}

void FNDIGeometryCollectionProxy::PreStage(const FNDIGpuComputePreStageContext& Context)
{
	check(SystemInstancesToProxyData.Contains(Context.GetSystemInstanceID()));

	FNDIGeometryCollectionData* ProxyData = SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());
	if (ProxyData != nullptr && ProxyData->AssetBuffer)
	{
		if (Context.GetSimStageData().bFirstStage)
		{
			FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

			// #todo(dmp): bounds buffer doesn't need to be updated each frame
			NDIGeometryCollectionLocal::UpdateInternalBuffer<FVector4f>(RHICmdList, ProxyData->AssetArrays->WorldTransformBuffer, ProxyData->AssetBuffer->WorldTransformBuffer);
			NDIGeometryCollectionLocal::UpdateInternalBuffer<FVector4f>(RHICmdList, ProxyData->AssetArrays->PrevWorldTransformBuffer, ProxyData->AssetBuffer->PrevWorldTransformBuffer);
			NDIGeometryCollectionLocal::UpdateInternalBuffer<FVector4f>(RHICmdList, ProxyData->AssetArrays->WorldInverseTransformBuffer, ProxyData->AssetBuffer->WorldInverseTransformBuffer);
			NDIGeometryCollectionLocal::UpdateInternalBuffer<FVector4f>(RHICmdList, ProxyData->AssetArrays->PrevWorldInverseTransformBuffer, ProxyData->AssetBuffer->PrevWorldInverseTransformBuffer);
			NDIGeometryCollectionLocal::UpdateInternalBuffer<FVector4f>(RHICmdList, ProxyData->AssetArrays->BoundsBuffer, ProxyData->AssetBuffer->BoundsBuffer);

			// build RDG buffer with per-element component transforms
			FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
			TArray<uint8>& DataToUpload = ProxyData->AssetBuffer->DataToUpload;
			if (DataToUpload.IsEmpty() && !ProxyData->AssetBuffer->ComponentRestTransformBuffer.IsValid())
			{
				// add dummy data so we can bind something to the buffer
				DataToUpload.SetNumZeroed(12);
			}

			if (!DataToUpload.IsEmpty())
			{
				const uint64 InitialDataSize = DataToUpload.Num() * DataToUpload.GetTypeSize();
				const uint64 BufferSize = Align(InitialDataSize, 16);

				FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateByteAddressDesc(BufferSize);
				ResizeBufferIfNeeded(GraphBuilder, ProxyData->AssetBuffer->ComponentRestTransformBuffer, BufferDesc, TEXT("NiagaraGeometryCollection"));

				GraphBuilder.QueueBufferUpload(
					GraphBuilder.RegisterExternalBuffer(ProxyData->AssetBuffer->ComponentRestTransformBuffer),
					DataToUpload.GetData(),
					InitialDataSize,
					ERDGInitialDataFlags::None
				);

				DataToUpload.Empty();
			}
		}
	}
}

//------------------------------------------------------------------------------------------------------------

UNiagaraDataInterfaceGeometryCollection::UNiagaraDataInterfaceGeometryCollection(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)	
{
	Proxy.Reset(new FNDIGeometryCollectionProxy());
}

bool UNiagaraDataInterfaceGeometryCollection::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIGeometryCollectionData* InstanceData = new (PerInstanceData) FNDIGeometryCollectionData();

	check(InstanceData);
	ResolveGeometryCollection(SystemInstance, InstanceData);
	InstanceData->Init(this, SystemInstance);
	
	return true;
}

ETickingGroup UNiagaraDataInterfaceGeometryCollection::CalculateTickGroup(const void* PerInstanceData) const
{
	if (const FNDIGeometryCollectionData* InstanceData = static_cast<const FNDIGeometryCollectionData*>(PerInstanceData))
	{
		return InstanceData->TickingGroup;
	}
	return NiagaraFirstTickGroup;
}

void UNiagaraDataInterfaceGeometryCollection::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIGeometryCollectionData* InstanceData = static_cast<FNDIGeometryCollectionData*>(PerInstanceData);

	InstanceData->Release();	
	InstanceData->~FNDIGeometryCollectionData();

	FNDIGeometryCollectionProxy* ThisProxy = GetProxyAs<FNDIGeometryCollectionProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			FNDIGeometryCollectionData* ProxyData = ThisProxy->SystemInstancesToProxyData.Find(InstanceID);

			if (ProxyData != nullptr && ProxyData->AssetArrays)
			{			
				ThisProxy->SystemInstancesToProxyData.Remove(InstanceID);
				delete ProxyData->AssetArrays;
			}		
		}
	);
}

bool UNiagaraDataInterfaceGeometryCollection::PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds)
{
	FNDIGeometryCollectionData* InstanceData = static_cast<FNDIGeometryCollectionData*>(PerInstanceData);
	ResolveGeometryCollection(SystemInstance, InstanceData);
	if (InstanceData && InstanceData->AssetBuffer && SystemInstance)
	{
		InstanceData->Update(this, SystemInstance);
	}
	return false;
}

bool UNiagaraDataInterfaceGeometryCollection::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	UNiagaraDataInterfaceGeometryCollection* OtherTyped = CastChecked<UNiagaraDataInterfaceGeometryCollection>(Destination);		

	OtherTyped->SourceMode = SourceMode;
#if WITH_EDITORONLY_DATA
	OtherTyped->PreviewCollection = PreviewCollection;
#endif
	OtherTyped->DefaultGeometryCollection = DefaultGeometryCollection;
	OtherTyped->GeometryCollectionActor = GeometryCollectionActor;
	OtherTyped->SourceComponent = SourceComponent;
	OtherTyped->GeometryCollectionUserParameter = GeometryCollectionUserParameter;
	OtherTyped->bIncludeIntermediateBones = bIncludeIntermediateBones;

	return true;
}

void UNiagaraDataInterfaceGeometryCollection::ResolveGeometryCollection(FNiagaraSystemInstance* SystemInstance, FNDIGeometryCollectionData* InstanceData)
{
	FNiagaraParameterDirectBinding<UObject*> CollectionParameterBinding;
	CollectionParameterBinding.Init(SystemInstance->GetInstanceParameters(), GeometryCollectionUserParameter.Parameter);
	UObject* UserParameter = CollectionParameterBinding.GetValue();
	
	InstanceData->ResolvedSource = FResolvedNiagaraGeometryCollection();

	switch (SourceMode)
	{
	case ENDIGeometryCollection_SourceMode::Source:
		ResolveGeometryCollectionFromDirectSource(InstanceData->ResolvedSource);
		break;
	case ENDIGeometryCollection_SourceMode::AttachParent:
		ResolveGeometryCollectionFromAttachParent(SystemInstance, InstanceData->ResolvedSource);
		break;
	case ENDIGeometryCollection_SourceMode::DefaultCollectionOnly:
		ResolveGeometryCollectionFromDefaultCollection(InstanceData->ResolvedSource);
		break;
	case ENDIGeometryCollection_SourceMode::ParameterBinding:
		ResolveGeometryCollectionFromParameterBinding(UserParameter, InstanceData->ResolvedSource);
		break;
	case ENDIGeometryCollection_SourceMode::Default:
	default:
		if (!ResolveGeometryCollectionFromDirectSource(InstanceData->ResolvedSource))
		{
			if (!ResolveGeometryCollectionFromParameterBinding(UserParameter, InstanceData->ResolvedSource))
			{
				if (!ResolveGeometryCollectionFromAttachParent(SystemInstance, InstanceData->ResolvedSource))
				{
					ResolveGeometryCollectionFromDefaultCollection(InstanceData->ResolvedSource);
				}
			}
		}
		break;
	}

#if WITH_EDITORONLY_DATA
	if (!InstanceData->ResolvedSource.Collection.IsValid() && !InstanceData->ResolvedSource.Component.IsValid() && (!SystemInstance || !SystemInstance->GetWorld()->IsGameWorld()))
	{
		// NOTE: We don't fall back on the preview mesh if we have a valid collection referenced
		InstanceData->ResolvedSource.Collection = PreviewCollection.LoadSynchronous();
	}
#endif
}

bool UNiagaraDataInterfaceGeometryCollection::ResolveGeometryCollectionFromDirectSource(FResolvedNiagaraGeometryCollection& ResolvedSource)
{
	if (::IsValid(SourceComponent))
	{
		ResolvedSource.Component = SourceComponent;
		return true;
	}
	if (GeometryCollectionActor.Get())
	{
		ResolvedSource.Component = GeometryCollectionActor->GetGeometryCollectionComponent();
	}
	return false;
}

bool UNiagaraDataInterfaceGeometryCollection::ResolveGeometryCollectionFromAttachParent(FNiagaraSystemInstance* SystemInstance, FResolvedNiagaraGeometryCollection& ResolvedSource)
{
	if (USceneComponent* AttachComponent = SystemInstance->GetAttachComponent())
	{
		// First, try to find the geometry collection component up the attachment hierarchy
		for (USceneComponent* Curr = AttachComponent; Curr; Curr = Curr->GetAttachParent())
		{
			UGeometryCollectionComponent* ParentComp = Cast<UGeometryCollectionComponent>(Curr);
			if (::IsValid(ParentComp))
			{
				ResolvedSource.Component = ParentComp;
				return true;
			}
		}

		// Next, try to find one in our outer chain
		UGeometryCollectionComponent* OuterComp = AttachComponent->GetTypedOuter<UGeometryCollectionComponent>();
		if (::IsValid(OuterComp))
		{
			ResolvedSource.Component = OuterComp;
			return true;
		}
		return ResolveGeometryCollectionFromActor(AttachComponent->GetAttachmentRootActor(), ResolvedSource);
	}
	return false;
}

bool UNiagaraDataInterfaceGeometryCollection::ResolveGeometryCollectionFromActor(AActor* Actor, FResolvedNiagaraGeometryCollection& ResolvedSource)
{
	if (Actor)
	{
		// Final fall-back, look for any component on our root actor or any of its parents
		if (AGeometryCollectionActor* GeoActor = Cast<AGeometryCollectionActor>(Actor))
		{
			ResolvedSource.Component = GeoActor->GetGeometryCollectionComponent();
			return true;
		}

		// Fall back on any valid component on the actor
		for (UActorComponent* ActorComp : Actor->GetComponents())
		{
			UGeometryCollectionComponent* Comp = Cast<UGeometryCollectionComponent>(ActorComp);
			if (::IsValid(Comp))
			{
				ResolvedSource.Component = Comp;
				return true;
			}
		}
	}
	return false;
}

bool UNiagaraDataInterfaceGeometryCollection::ResolveGeometryCollectionFromDefaultCollection(FResolvedNiagaraGeometryCollection& ResolvedSource)
{
	ResolvedSource.Collection = DefaultGeometryCollection;
	return true;
}

bool UNiagaraDataInterfaceGeometryCollection::ResolveGeometryCollectionFromParameterBinding(UObject* ParameterBindingValue, FResolvedNiagaraGeometryCollection& ResolvedSource)
{
	if (ParameterBindingValue == nullptr)
	{
		return false;
	}
	if (ResolveGeometryCollectionFromActor(Cast<AActor>(ParameterBindingValue), ResolvedSource))
	{
		return true;
	}
	if (UGeometryCollectionComponent* UserComponent = Cast<UGeometryCollectionComponent>(ParameterBindingValue))
	{
		ResolvedSource.Component = UserComponent;
		return true;
	}
	if (UGeometryCollection* UserCollection = Cast<UGeometryCollection>(ParameterBindingValue))
	{
		ResolvedSource.Collection = UserCollection;
		return true;
	}
	return false;
}

bool UNiagaraDataInterfaceGeometryCollection::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance, float DeltaSeconds)
{
	FNDIGeometryCollectionData* InstanceData = static_cast<FNDIGeometryCollectionData*>(PerInstanceData);
	if (InstanceData && InstanceData->bHasPendingComponentTransformUpdate && InstanceData->ResolvedSource.Component.IsValid())
	{
		InstanceData->ResolvedSource.Component->SetLocalRestTransforms(InstanceData->AssetArrays->ComponentRestTransformBuffer, !bIncludeIntermediateBones);
	}
	return false;
}

bool UNiagaraDataInterfaceGeometryCollection::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	const UNiagaraDataInterfaceGeometryCollection* OtherTyped = CastChecked<const UNiagaraDataInterfaceGeometryCollection>(Other);

	return  OtherTyped->SourceMode == SourceMode &&
#if WITH_EDITORONLY_DATA
			OtherTyped->PreviewCollection == PreviewCollection &&
#endif
			OtherTyped->DefaultGeometryCollection == DefaultGeometryCollection &&
			OtherTyped->GeometryCollectionActor == GeometryCollectionActor &&
			OtherTyped->SourceComponent == SourceComponent &&
			OtherTyped->GeometryCollectionUserParameter == GeometryCollectionUserParameter &&
			OtherTyped->bIncludeIntermediateBones == bIncludeIntermediateBones;
}

void UNiagaraDataInterfaceGeometryCollection::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA

void UNiagaraDataInterfaceGeometryCollection::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDIGeometryCollectionLocal::GetClosestPointNoNormalName;
		Sig.bSupportsCPU = false;
		Sig.bMemberFunction = true;
		Sig.FunctionVersion = NDIGeometryCollectionLocal::FGeometryCollectionDIFunctionVersion::LatestVersion;
		Sig.Description = LOCTEXT("SigNoNormalNameDescription", "Returns the closest point on the surface of the geometry collection.");
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Geometry Collection DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Delta Time")), LOCTEXT("ClosestPointNoNormal_DeltaTimeTooltip", "Current delta time to compute the returned velocity"));
		Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Time Fraction")), LOCTEXT("ClosestPointNoNormal_TimeFractionTooltip", "Lerps the returned closest position between the current frame (1.0) and the previous frame (0.0)."));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Closest Distance")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Closest Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Closest Velocity")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDIGeometryCollectionLocal::GetNumElementsName;
		Sig.bMemberFunction = true;
		Sig.FunctionVersion = NDIGeometryCollectionLocal::FGeometryCollectionDIFunctionVersion::LatestVersion;
		Sig.Description = LOCTEXT("SigNumElementsDescription", "Returns the numbers of elements in the geometry collection. Unless 'IncludeIntermediateBones' is set, this only counts leaf geometries.");
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Geometry Collection DI")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Count")));
		NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDIGeometryCollectionLocal::GetElementBoundsName;
		Sig.bMemberFunction = true;
		Sig.FunctionVersion = NDIGeometryCollectionLocal::FGeometryCollectionDIFunctionVersion::LatestVersion;
		Sig.Description = LOCTEXT("GetElementBoundsNameDescription", "Returns the current bounding box and size for the given element. The values are relative to the geometry component root.");
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Geometry Collection DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Bounding Box Center")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Bounding Box Size")));
		NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDIGeometryCollectionLocal::GetTransformComponentName;
		Sig.bMemberFunction = true;
		Sig.FunctionVersion = NDIGeometryCollectionLocal::FGeometryCollectionDIFunctionVersion::LatestVersion;
		Sig.Description = LOCTEXT("SigGetTransformComponentNameDescription", "Returns the transform for the given element index relative to the root of the geometry collection.");
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Geometry Collection DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Translation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDIGeometryCollectionLocal::SetTransformComponentName;
		Sig.bSupportsGPU = false; //TODO: add gpu support
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.FunctionVersion = NDIGeometryCollectionLocal::FGeometryCollectionDIFunctionVersion::LatestVersion;
		Sig.Description = LOCTEXT("SigSetTransformComponentNameDescription", "Sets the transform for the given element index relative to the root of the geometry collection.");
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Geometry Collection DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Translation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDIGeometryCollectionLocal::SetTransformWorldName;
		Sig.bSupportsGPU = false; //TODO: add gpu support
		Sig.bMemberFunction = true;
		Sig.bRequiresExecPin = true;
		Sig.FunctionVersion = NDIGeometryCollectionLocal::FGeometryCollectionDIFunctionVersion::LatestVersion;
		Sig.Description = LOCTEXT("SigSetTransformWorldNameDescription", "Sets the transform for the given element index in world space.");
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Geometry Collection DI")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Element Index")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)

		OutFunctions.Add(Sig);
	}

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = NDIGeometryCollectionLocal::GetComponentWSTransformName;
		Sig.bMemberFunction = true;
		Sig.FunctionVersion = NDIGeometryCollectionLocal::FGeometryCollectionDIFunctionVersion::LatestVersion;
		Sig.Description = LOCTEXT("SigGetActorTransformNameDescription", "Returns the transform for the geometry collection component (or its owning actor) in world space.");
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Geometry Collection DI")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("Position")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetQuatDef(), TEXT("Rotation")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Scale")));
		NIAGARA_ADD_FUNCTION_SOURCE_INFO(Sig)

		OutFunctions.Add(Sig);
	}
}
#endif

void UNiagaraDataInterfaceGeometryCollection::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIGeometryCollectionLocal;

	if ( BindingInfo.Name == GetNumElementsName )
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceGeometryCollection::GetNumGeometryElements);
	}
	else if ( BindingInfo.Name == GetElementBoundsName )
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceGeometryCollection::GetElementBounds);
	}
	else if ( BindingInfo.Name == GetTransformComponentName )
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceGeometryCollection::GetElementTransformCS);
	}
	else if ( BindingInfo.Name == SetTransformComponentName )
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceGeometryCollection::SetElementTransformCS);
	}
	else if ( BindingInfo.Name == SetTransformWorldName )
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceGeometryCollection::SetElementTransformWS);
	}
	else if ( BindingInfo.Name == GetComponentWSTransformName )
	{
		OutFunc = FVMExternalFunction::CreateUObject(this, &UNiagaraDataInterfaceGeometryCollection::GetActorTransform);
	}
}

void UNiagaraDataInterfaceGeometryCollection::GetNumGeometryElements(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIGeometryCollectionData> InstanceData(Context);
	FNDIOutputParam<int32> OutCount(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutCount.SetAndAdvance(InstanceData.Get()->AssetArrays->NumPieces);
	}
}

void UNiagaraDataInterfaceGeometryCollection::GetElementBounds(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIGeometryCollectionData> InstanceData(Context);
	FNDIInputParam<int32> InElement(Context);
	FNDIOutputParam<FVector3f> OutCenter(Context);
	FNDIOutputParam<FVector3f> OutBoundingBoxSize(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 ElementIndex = InElement.GetAndAdvance();
		int32 TransformIndex = INDEX_NONE;
		if (InstanceData.Get()->AssetArrays->ElementIndexToTransformBufferMapping.IsValidIndex(ElementIndex))
		{
			TransformIndex = InstanceData.Get()->AssetArrays->ElementIndexToTransformBufferMapping[ElementIndex];
		}
		if (InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer.IsValidIndex(TransformIndex))
		{
			const FTransform& Transform = InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer[TransformIndex];
			FVector3f Center(Transform.GetTranslation());
			FVector4f BoundsExtent = InstanceData.Get()->AssetArrays->BoundsBuffer[ElementIndex];

			OutCenter.SetAndAdvance(Center);
			OutBoundingBoxSize.SetAndAdvance(FVector3f(BoundsExtent));
		}
		else
		{
			OutCenter.SetAndAdvance(FVector3f::ZeroVector);
			OutBoundingBoxSize.SetAndAdvance(FVector3f::ZeroVector);
		}
	}
}

void UNiagaraDataInterfaceGeometryCollection::GetElementTransformCS(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIGeometryCollectionData> InstanceData(Context);
	FNDIInputParam<int32> InElement(Context);
	FNDIOutputParam<FVector3f> OutTranslation(Context);
	FNDIOutputParam<FQuat4f> OutRotation(Context);
	FNDIOutputParam<FVector3f> OutScale(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 ElementIndex = InElement.GetAndAdvance();
		int32 TransformIndex = INDEX_NONE;
		if (InstanceData.Get()->AssetArrays->ElementIndexToTransformBufferMapping.IsValidIndex(ElementIndex))
		{
			TransformIndex = InstanceData.Get()->AssetArrays->ElementIndexToTransformBufferMapping[ElementIndex];
		}
		if (InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer.IsValidIndex(TransformIndex))
		{
			const FTransform& Transform = InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer[TransformIndex];
			OutTranslation.SetAndAdvance(FVector3f(Transform.GetTranslation()));
			OutRotation.SetAndAdvance(FQuat4f(Transform.GetRotation().GetNormalized()));
			OutScale.SetAndAdvance(FVector3f(Transform.GetScale3D()));
		}
		else
		{
			OutTranslation.SetAndAdvance(FVector3f::ZeroVector);
			OutRotation.SetAndAdvance(FQuat4f::Identity);
			OutScale.SetAndAdvance(FVector3f::OneVector);
		}
	}
}

void UNiagaraDataInterfaceGeometryCollection::SetElementTransformWS(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIGeometryCollectionData> InstanceData(Context);
	FNDIInputParam<int32> InElement(Context);
	FNDIInputParam<FNiagaraPosition> InTranslation(Context);
	FNDIInputParam<FQuat4f> InRotation(Context);
	FNDIInputParam<FVector3f> InScale(Context);

	const FTransform& InverseRootTransform = InstanceData.Get()->RootTransform.Inverse();
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 ElementIndex = InElement.GetAndAdvance();
		FVector Translation(InTranslation.GetAndAdvance());
		FQuat Rotation(InRotation.GetAndAdvance());
		FVector Scale(InScale.GetAndAdvance());

		if (InstanceData.Get()->AssetArrays->ElementIndexToTransformBufferMapping.IsValidIndex(ElementIndex))
		{
			int32 TransformIndex = InstanceData.Get()->AssetArrays->ElementIndexToTransformBufferMapping[ElementIndex];
			if (InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer.IsValidIndex(TransformIndex))
			{
				FTransform Transform(Rotation, Translation, Scale);
				// the geometry component wants all the transforms to be in localspace, so we need to remove the component root transform (which also removes the LWC tile offset)
				Transform *= InverseRootTransform;
				if (!Transform.Equals(InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer[TransformIndex]))
				{
					InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer[TransformIndex] = Transform;
					InstanceData.Get()->bHasPendingComponentTransformUpdate = true;
				}
			}
		}
	}
}

void UNiagaraDataInterfaceGeometryCollection::SetElementTransformCS(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIGeometryCollectionData> InstanceData(Context);
	FNDIInputParam<int32> InElement(Context);
	FNDIInputParam<FVector3f> InTranslation(Context);
	FNDIInputParam<FQuat4f> InRotation(Context);
	FNDIInputParam<FVector3f> InScale(Context);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		int32 ElementIndex = InElement.GetAndAdvance();
		FVector Translation(InTranslation.GetAndAdvance());
		FQuat Rotation(InRotation.GetAndAdvance());
		FVector Scale(InScale.GetAndAdvance());

		if (InstanceData.Get()->AssetArrays->ElementIndexToTransformBufferMapping.IsValidIndex(ElementIndex))
		{
			int32 TransformIndex = InstanceData.Get()->AssetArrays->ElementIndexToTransformBufferMapping[ElementIndex];
			if (InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer.IsValidIndex(TransformIndex))
			{
				FTransform NewTransform(Rotation, Translation, Scale);
				FTransform& CurrentTransform = InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer[TransformIndex];
				if (!CurrentTransform.Equals(NewTransform))
				{
					InstanceData.Get()->AssetArrays->ComponentRestTransformBuffer[TransformIndex] = NewTransform;
					InstanceData.Get()->bHasPendingComponentTransformUpdate = true;
				}
			}
		}
	}
}

void UNiagaraDataInterfaceGeometryCollection::GetActorTransform(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIGeometryCollectionData> InstanceData(Context);
	FNDIOutputParam<FNiagaraPosition> OutPosition(Context);
	FNDIOutputParam<FQuat4f> OutRotation(Context);
	FNDIOutputParam<FVector3f> OutScale(Context);

	const FTransform& Transform = InstanceData.Get()->RootTransform;
	FNiagaraPosition Location(Transform.GetTranslation());
	FQuat4f Rotation(Transform.GetRotation().GetNormalized());
	FVector3f Scale(Transform.GetScale3D());
	
	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		OutPosition.SetAndAdvance(Location);
		OutRotation.SetAndAdvance(Rotation);
		OutScale.SetAndAdvance(Scale);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceGeometryCollection::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	if (FunctionInfo.DefinitionName == NDIGeometryCollectionLocal::GetClosestPointNoNormalName ||
		FunctionInfo.DefinitionName == NDIGeometryCollectionLocal::GetNumElementsName ||
		FunctionInfo.DefinitionName == NDIGeometryCollectionLocal::GetComponentWSTransformName ||
		FunctionInfo.DefinitionName == NDIGeometryCollectionLocal::GetElementBoundsName ||
		FunctionInfo.DefinitionName == NDIGeometryCollectionLocal::GetTransformComponentName)
	{
		return true;
	}

	return false;
}



bool UNiagaraDataInterfaceGeometryCollection::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	// upgrade old functions to the latest version
	if (FunctionSignature.FunctionVersion < NDIGeometryCollectionLocal::FGeometryCollectionDIFunctionVersion::AddedElementIndexOutput)
	{
		TArray<FNiagaraFunctionSignature> AllFunctions;
		GetFunctionsInternal(AllFunctions);
		for (const FNiagaraFunctionSignature& Sig : AllFunctions)
		{
			if (FunctionSignature.Name == Sig.Name && Sig.Name == NDIGeometryCollectionLocal::GetClosestPointNoNormalName)
			{
				FunctionSignature = Sig;
				return true;
			}
		}
	}

	return false;
}

bool UNiagaraDataInterfaceGeometryCollection::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIGeometryCollectionLocal::TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

void UNiagaraDataInterfaceGeometryCollection::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs = { {TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol}, };
	AppendTemplateHLSL(OutHLSL, NDIGeometryCollectionLocal::TemplateShaderFilePath, TemplateArgs);
}
#endif

void UNiagaraDataInterfaceGeometryCollection::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceGeometryCollection::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNDIGeometryCollectionProxy& InterfaceProxy = Context.GetProxy<FNDIGeometryCollectionProxy>();
	FNDIGeometryCollectionData& ProxyData = InterfaceProxy.SystemInstancesToProxyData.FindChecked(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters = Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->BoundsMin							= ProxyData.BoundsOrigin - ProxyData.BoundsExtent;
	ShaderParameters->BoundsMax							= ProxyData.BoundsOrigin + ProxyData.BoundsExtent;
	ShaderParameters->NumPieces							= ProxyData.AssetBuffer->NumPieces;
	ShaderParameters->RootTransform_Translation			= FVector3f(ProxyData.RootTransform.GetTranslation());
	ShaderParameters->RootTransform_Rotation			= FQuat4f(ProxyData.RootTransform.GetRotation());
	ShaderParameters->RootTransform_Scale				= FVector3f(ProxyData.RootTransform.GetScale3D());
	ShaderParameters->WorldTransformBuffer				= FNiagaraRenderer::GetSrvOrDefaultFloat4(ProxyData.AssetBuffer->WorldTransformBuffer.SRV);
	ShaderParameters->PrevWorldTransformBuffer			= FNiagaraRenderer::GetSrvOrDefaultFloat4(ProxyData.AssetBuffer->PrevWorldTransformBuffer.SRV);
	ShaderParameters->WorldInverseTransformBuffer		= FNiagaraRenderer::GetSrvOrDefaultFloat4(ProxyData.AssetBuffer->WorldInverseTransformBuffer.SRV);
	ShaderParameters->PrevWorldInverseTransformBuffer	= FNiagaraRenderer::GetSrvOrDefaultFloat4(ProxyData.AssetBuffer->PrevWorldInverseTransformBuffer.SRV);
	ShaderParameters->BoundsBuffer						= FNiagaraRenderer::GetSrvOrDefaultFloat4(ProxyData.AssetBuffer->BoundsBuffer.SRV);

	if (ensure(ProxyData.AssetBuffer->ComponentRestTransformBuffer))
	{
		FRDGBuilder& GraphBuilder = Context.GetGraphBuilder();
		FRDGBufferRef RDGBuffer = GraphBuilder.RegisterExternalBuffer(ProxyData.AssetBuffer->ComponentRestTransformBuffer);
		ShaderParameters->ElementTransforms	= GraphBuilder.CreateSRV(RDGBuffer);
	}
}

void UNiagaraDataInterfaceGeometryCollection::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FNDIGeometryCollectionData* GameThreadData = static_cast<FNDIGeometryCollectionData*>(PerInstanceData);
	FNDIGeometryCollectionData* RenderThreadData = static_cast<FNDIGeometryCollectionData*>(DataForRenderThread);

	if (GameThreadData && RenderThreadData)
	{		
		RenderThreadData->AssetBuffer = GameThreadData->AssetBuffer;				

		RenderThreadData->AssetArrays = new FNDIGeometryCollectionArrays();
		RenderThreadData->AssetArrays->CopyFrom(GameThreadData->AssetArrays);		
		RenderThreadData->TickingGroup = GameThreadData->TickingGroup;
		RenderThreadData->BoundsOrigin = GameThreadData->BoundsOrigin;
		RenderThreadData->BoundsExtent = GameThreadData->BoundsExtent;
		RenderThreadData->RootTransform = GameThreadData->RootTransform;

		if (GameThreadData->bNeedsRenderUpdate)
		{
			GameThreadData->bNeedsRenderUpdate = false;
			
			constexpr int32 TransformGpuSize = 10 * 4;
			const int32 BufferSize = RenderThreadData->AssetArrays->ElementIndexToTransformBufferMapping.Num() * TransformGpuSize;
			RenderThreadData->AssetBuffer->DataToUpload.SetNumUninitialized(BufferSize);
			float* OutFloats = reinterpret_cast<float*>(RenderThreadData->AssetBuffer->DataToUpload.GetData());
			for (int32 Index : RenderThreadData->AssetArrays->ElementIndexToTransformBufferMapping)
			{
				const FTransform& Transform = RenderThreadData->AssetArrays->ComponentRestTransformBuffer[Index];
			
				const FVector3f Translation(Transform.GetTranslation());
				const FQuat4f Rotation(Transform.GetRotation());
				const FVector3f Scale(Transform.GetScale3D());

				FMemory::Memcpy(&OutFloats[0], &Translation, sizeof(FVector3f));
				FMemory::Memcpy(&OutFloats[3], &Rotation, sizeof(FQuat4f));
				FMemory::Memcpy(&OutFloats[7], &Scale, sizeof(FVector3f));
				OutFloats += 10;
			}
		}
	}
	check(Proxy);
}

#undef LOCTEXT_NAMESPACE
