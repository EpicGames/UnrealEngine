// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfacePhysicsField.h"
#include "Engine/World.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraRenderer.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"
#include "PhysicsField/PhysicsFieldComponent.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataInterfacePhysicsField)

#define LOCTEXT_NAMESPACE "NiagaraDataInterfacePhysicsField"
DEFINE_LOG_CATEGORY_STATIC(LogPhysicsField, Log, All);

struct FNiagaraPhysicsFieldDIFunctionVersion
{
	enum Type
	{
		InitialVersion = 0,
		LargeWorldCoordinates = 1,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};

//------------------------------------------------------------------------------------------------------------

namespace NDIPhysicsFieldLocal
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderInstanceParameters, )
		SHADER_PARAMETER(int,				bClipmapAvailable)
		SHADER_PARAMETER_ARRAY(FIntVector4, TargetMappings, [MAX_PHYSICS_FIELD_TARGETS])
		SHADER_PARAMETER_SRV(Buffer<float>,	ClipmapBuffer)
		SHADER_PARAMETER(FVector3f,			ClipmapCenter)
		SHADER_PARAMETER(float, 			ClipmapDistance)
		SHADER_PARAMETER(int, 				ClipmapResolution)
		SHADER_PARAMETER(int, 				ClipmapExponent)
		SHADER_PARAMETER(int, 				ClipmapCount)
		SHADER_PARAMETER(int,				TargetCount)
		SHADER_PARAMETER_ARRAY(FIntVector4, FieldTargets, [MAX_PHYSICS_FIELD_TARGETS])
		SHADER_PARAMETER_SRV(Buffer<float>,	NodesParams)
		SHADER_PARAMETER_SRV(Buffer<int>,	NodesOffsets)
		SHADER_PARAMETER_SRV(Buffer<int>,	TargetsOffsets)
		SHADER_PARAMETER(float,				TimeSeconds)
	END_SHADER_PARAMETER_STRUCT()

	static const TCHAR* TemplateShaderFilePath = TEXT("/Plugin/Experimental/ChaosNiagara/NiagaraDataInterfacePhysicsField.ush");

	static const FName SamplePhysicsVectorFieldName("SamplePhysicsVectorField");
	static const FName SamplePhysicsScalarFieldName("SamplePhysicsScalarField");
	static const FName SamplePhysicsIntegerFieldName("SamplePhysicsIntegerField");

	static const FName EvalPhysicsVectorFieldName("EvalPhysicsVectorField");
	static const FName EvalPhysicsScalarFieldName("EvalPhysicsScalarField");
	static const FName EvalPhysicsIntegerFieldName("EvalPhysicsIntegerField");

	static const FName GetPhysicsFieldResolutionName("GetPhysicsFieldResolution");
	static const FName GetPhysicsFieldBoundsName("GetPhysicsFieldBounds");
}

//------------------------------------------------------------------------------------------------------------

void FNDIPhysicsFieldData::Release()
{
	FieldResource = nullptr;
	FieldCommands.Empty();
}

void FNDIPhysicsFieldData::Init(FNiagaraSystemInstance* SystemInstance)
{
	Release();
	if (SystemInstance != nullptr)
	{
		UWorld* World = SystemInstance->GetWorld();
		if (World)
		{
			TimeSeconds = World->GetTimeSeconds();

			UPhysicsFieldComponent* FieldComponent = World->PhysicsField;
			if (FieldComponent && FieldComponent->FieldInstance && FieldComponent->FieldInstance->FieldResource)
			{
				FieldResource = FieldComponent->FieldInstance->FieldResource;
			}
		}
		LWCConverter = SystemInstance->GetLWCConverter();
	}
}

void FNDIPhysicsFieldData::Update(FNiagaraSystemInstance* SystemInstance)
{
	if (SystemInstance != nullptr)
	{
		UWorld* World = SystemInstance->GetWorld();
		if (World)
		{	
			TimeSeconds = World->GetTimeSeconds();

			UPhysicsFieldComponent* FieldComponent = World->PhysicsField;
			if (FieldComponent && FieldComponent->FieldInstance)
			{
				FieldCommands = FieldComponent->FieldInstance->FieldCommands;
			}
		}
	}
}

//------------------------------------------------------------------------------------------------------------

void FNDIPhysicsFieldProxy::ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance)
{
	FNDIFieldRenderData* SourceData = static_cast<FNDIFieldRenderData*>(PerInstanceData);
	FNDIFieldRenderData* TargetData = &(SystemInstancesToProxyData.FindOrAdd(Instance));

	ensure(TargetData);
	if (TargetData && SourceData && SourceData->FieldResource)
	{
		TargetData->FieldResource = SourceData->FieldResource;
		TargetData->TimeSeconds = SourceData->TimeSeconds;
	}
	SourceData->~FNDIFieldRenderData();
}

void FNDIPhysicsFieldProxy::InitializePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());
	SystemInstancesToProxyData.Add(SystemInstance);
}

void FNDIPhysicsFieldProxy::DestroyPerInstanceData(const FNiagaraSystemInstanceID& SystemInstance)
{
	check(IsInRenderingThread());
	SystemInstancesToProxyData.Remove(SystemInstance);
}

//------------------------------------------------------------------------------------------------------------

UNiagaraDataInterfacePhysicsField::UNiagaraDataInterfacePhysicsField()
{
	Proxy.Reset(new FNDIPhysicsFieldProxy());
}

bool UNiagaraDataInterfacePhysicsField::InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIPhysicsFieldData* InstanceData = new (PerInstanceData) FNDIPhysicsFieldData();

	check(InstanceData);
	if (SystemInstance)
	{
		InstanceData->Init(SystemInstance);
	}

	return true;
}

void UNiagaraDataInterfacePhysicsField::DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	FNDIPhysicsFieldData* InstanceData = static_cast<FNDIPhysicsFieldData*>(PerInstanceData);

	InstanceData->Release();
	InstanceData->~FNDIPhysicsFieldData();

	FNDIPhysicsFieldProxy* ThisProxy = GetProxyAs<FNDIPhysicsFieldProxy>();
	ENQUEUE_RENDER_COMMAND(FNiagaraDIDestroyInstanceData) (
		[ThisProxy, InstanceID = SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
	{
		ThisProxy->SystemInstancesToProxyData.Remove(InstanceID);
	}
	);
}

bool UNiagaraDataInterfacePhysicsField::PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float InDeltaSeconds)
{
	FNDIPhysicsFieldData* InstanceData = static_cast<FNDIPhysicsFieldData*>(PerInstanceData);
	if (InstanceData && SystemInstance)
	{
		InstanceData->Update(SystemInstance);
	}
	return false;
}

bool UNiagaraDataInterfacePhysicsField::CopyToInternal(UNiagaraDataInterface* Destination) const
{
	if (!Super::CopyToInternal(Destination))
	{
		return false;
	}

	return true;
}

bool UNiagaraDataInterfacePhysicsField::Equals(const UNiagaraDataInterface* Other) const
{
	if (!Super::Equals(Other))
	{
		return false;
	}
	return true;
}

void UNiagaraDataInterfacePhysicsField::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags DIFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), DIFlags);

		ENiagaraTypeRegistryFlags FieldFlags =
			ENiagaraTypeRegistryFlags::AllowAnyVariable |
			ENiagaraTypeRegistryFlags::AllowParameter;
		
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<EFieldVectorType>()), FieldFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<EFieldScalarType>()), FieldFlags);
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(StaticEnum<EFieldIntegerType>()), FieldFlags);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfacePhysicsField::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIPhysicsFieldLocal;

	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePhysicsVectorFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Field")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(StaticEnum<EFieldVectorType>(), TEXT("Target Type")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Vector Value")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePhysicsScalarFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Field")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(StaticEnum<EFieldScalarType>(), TEXT("Target Type")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Scalar Value")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = SamplePhysicsIntegerFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Field")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(StaticEnum<EFieldIntegerType>(), TEXT("Target Type")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Integer Value")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = EvalPhysicsVectorFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Field")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(StaticEnum<EFieldVectorType>(), TEXT("Target Type")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Vector Value")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = EvalPhysicsScalarFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Field")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(StaticEnum<EFieldScalarType>(), TEXT("Target Type")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Scalar Value")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = EvalPhysicsIntegerFieldName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Field")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), TEXT("World Position")));
		Sig.Inputs.Add(FNiagaraVariable(StaticEnum<EFieldIntegerType>(), TEXT("Target Type")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Integer Value")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPhysicsFieldResolutionName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Field")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Field Resolution")));

		OutFunctions.Add(Sig);
	}
	{
		FNiagaraFunctionSignature Sig;
		Sig.Name = GetPhysicsFieldBoundsName;
		Sig.bMemberFunction = true;
		Sig.bRequiresContext = false;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("Physics Field")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Min Bounds"))); //TODO (LWC) not sure what to do with these bounds, should they be converted as well?
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetVec3Def(), TEXT("Max Bounds")));

		OutFunctions.Add(Sig);
	}
}
#endif

DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, SamplePhysicsVectorField);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, SamplePhysicsScalarField);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, SamplePhysicsIntegerField);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, GetPhysicsFieldResolution);
DEFINE_NDI_DIRECT_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, GetPhysicsFieldBounds);

void UNiagaraDataInterfacePhysicsField::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIPhysicsFieldLocal;
	if (BindingInfo.Name == SamplePhysicsVectorFieldName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, SamplePhysicsVectorField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SamplePhysicsScalarFieldName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, SamplePhysicsScalarField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == SamplePhysicsIntegerFieldName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, SamplePhysicsIntegerField)::Bind(this, OutFunc);
	}
	if (BindingInfo.Name == EvalPhysicsVectorFieldName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, SamplePhysicsVectorField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == EvalPhysicsScalarFieldName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, SamplePhysicsScalarField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == EvalPhysicsIntegerFieldName)
	{
		check(BindingInfo.GetNumInputs() == 5 && BindingInfo.GetNumOutputs() == 1);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, SamplePhysicsIntegerField)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetPhysicsFieldResolutionName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 3);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, GetPhysicsFieldResolution)::Bind(this, OutFunc);
	}
	else if (BindingInfo.Name == GetPhysicsFieldBoundsName)
	{
		check(BindingInfo.GetNumInputs() == 1 && BindingInfo.GetNumOutputs() == 6);
		NDI_FUNC_BINDER(UNiagaraDataInterfacePhysicsField, GetPhysicsFieldBounds)::Bind(this, OutFunc);
	}
}

void UNiagaraDataInterfacePhysicsField::GetPhysicsFieldResolution(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIPhysicsFieldData> InstData(Context);

	VectorVM::FExternalFuncRegisterHandler<float> OutDimensionX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutDimensionY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutDimensionZ(Context);

	const FIntVector FieldDimension = (InstData && InstData->FieldResource) ? FIntVector(InstData->FieldResource->FieldInfos.ClipmapResolution, 
		InstData->FieldResource->FieldInfos.ClipmapResolution, InstData->FieldResource->FieldInfos.ClipmapResolution) : FIntVector(1, 1, 1);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*OutDimensionX.GetDest() = FieldDimension.X;
		*OutDimensionY.GetDest() = FieldDimension.Y;
		*OutDimensionZ.GetDest() = FieldDimension.Z;

		OutDimensionX.Advance();
		OutDimensionY.Advance();
		OutDimensionZ.Advance();
	}
}

void UNiagaraDataInterfacePhysicsField::GetPhysicsFieldBounds(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIPhysicsFieldData> InstData(Context);

	VectorVM::FExternalFuncRegisterHandler<float> OutMinX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMinY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMinZ(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMaxX(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMaxY(Context);
	VectorVM::FExternalFuncRegisterHandler<float> OutMaxZ(Context);

	const FVector MinBound = (InstData && InstData->FieldResource) ? InstData->FieldResource->FieldInfos.ClipmapCenter -
		FVector(InstData->FieldResource->FieldInfos.ClipmapDistance) : FVector(0, 0, 0);
	const FVector MaxBound = (InstData && InstData->FieldResource) ? InstData->FieldResource->FieldInfos.ClipmapCenter +
		FVector(InstData->FieldResource->FieldInfos.ClipmapDistance) : FVector(0, 0, 0);

	for (int32 i = 0; i < Context.GetNumInstances(); ++i)
	{
		*OutMinX.GetDest() = MinBound.X;
		*OutMinY.GetDest() = MinBound.Y;
		*OutMinZ.GetDest() = MinBound.Z;
		*OutMaxX.GetDest() = MaxBound.X;
		*OutMaxY.GetDest() = MaxBound.Y;
		*OutMaxZ.GetDest() = MaxBound.Z;

		OutMinX.Advance();
		OutMinY.Advance();
		OutMinZ.Advance();
		OutMaxX.Advance();
		OutMaxY.Advance();
		OutMaxZ.Advance();
	}
}

void UNiagaraDataInterfacePhysicsField::SamplePhysicsVectorField(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIPhysicsFieldData> InstData(Context);

	// Inputs 
	FNDIInputParam<FNiagaraPosition> SamplePositionParam(Context);
	FNDIInputParam<EFieldVectorType> VectorTargetParam(Context);

	// Outputs...
	FNDIOutputParam<FVector3f> OutVectorFieldParam(Context);

	if (InstData)
	{
		FFieldExecutionDatas ExecutionDatas;
		ExecutionDatas.SamplePositions.Init(FVector(0, 0, 0), Context.GetNumInstances());

		EFieldVectorType VectorTarget = Vector_TargetMax;

		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			ExecutionDatas.SamplePositions[InstanceIdx] = InstData->LWCConverter.ConvertSimulationPositionToWorld(SamplePositionParam.GetAndAdvance());
			VectorTarget = VectorTargetParam.GetAndAdvance();
		}
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, Context.GetNumInstances());

		TArray<FVector>& SampleResults = ExecutionDatas.VectorResults[(uint8)EFieldCommandResultType::FinalResult];
		SampleResults.Init(FVector::ZeroVector, Context.GetNumInstances());

		TArray<FVector> SampleMax;
		SampleMax.Init(FVector::ZeroVector, Context.GetNumInstances());

		FFieldContext FieldContext{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			InstData->TimeSeconds
		};

		const EFieldPhysicsType PhysicsType = GetFieldTargetTypes(Field_Output_Vector)[VectorTarget];
		EvaluateFieldVectorNodes(InstData->FieldCommands, PhysicsType, FieldContext, SampleResults, SampleMax);

		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			OutVectorFieldParam.SetAndAdvance(FVector3f(SampleMax[InstanceIdx]));
		}
	}
	else
	{
		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			OutVectorFieldParam.SetAndAdvance(FVector3f::ZeroVector);
		}
	}
}

void UNiagaraDataInterfacePhysicsField::SamplePhysicsIntegerField(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIPhysicsFieldData> InstData(Context);

	// Inputs 
	FNDIInputParam<FNiagaraPosition> SamplePositionParam(Context);
	FNDIInputParam<EFieldIntegerType> IntegerTargetParam(Context);

	// Outputs...
	FNDIOutputParam<int32> OutIntegerFieldParam(Context);

	if (InstData)
	{
		FFieldExecutionDatas ExecutionDatas;
		ExecutionDatas.SamplePositions.Init(FVector(0, 0, 0), Context.GetNumInstances());

		EFieldIntegerType IntegerTarget = Integer_TargetMax;

		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			ExecutionDatas.SamplePositions[InstanceIdx] = InstData->LWCConverter.ConvertSimulationPositionToWorld(SamplePositionParam.GetAndAdvance());
			IntegerTarget = IntegerTargetParam.GetAndAdvance();
		}
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, Context.GetNumInstances());

		TArray<int32>& SampleResults = ExecutionDatas.IntegerResults[(uint8)EFieldCommandResultType::FinalResult];
		SampleResults.Init(0, Context.GetNumInstances());

		TArray<int32> SampleMax;
		SampleMax.Init(0, Context.GetNumInstances());

		FFieldContext FieldContext{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			InstData->TimeSeconds
		};

		const EFieldPhysicsType PhysicsType = GetFieldTargetTypes(Field_Output_Integer)[IntegerTarget];
		EvaluateFieldIntegerNodes(InstData->FieldCommands, PhysicsType, FieldContext, SampleResults, SampleMax);

		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			OutIntegerFieldParam.SetAndAdvance(SampleMax[InstanceIdx]);
		}
	}
	else
	{
		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			OutIntegerFieldParam.SetAndAdvance(0.0);
		}
	}
}

void UNiagaraDataInterfacePhysicsField::SamplePhysicsScalarField(FVectorVMExternalFunctionContext& Context)
{
	VectorVM::FUserPtrHandler<FNDIPhysicsFieldData> InstData(Context);

	// Inputs 
	FNDIInputParam<FNiagaraPosition> SamplePositionParam(Context);
	FNDIInputParam<EFieldScalarType> ScalarTargetParam(Context);

	// Outputs...
	FNDIOutputParam<float> OutScalarFieldParam(Context);

	if (InstData)
	{
		FFieldExecutionDatas ExecutionDatas;
		ExecutionDatas.SamplePositions.Init(FVector(0, 0, 0), Context.GetNumInstances());

		EFieldScalarType ScalarTarget = Scalar_TargetMax;

		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			ExecutionDatas.SamplePositions[InstanceIdx] = InstData->LWCConverter.ConvertSimulationPositionToWorld(SamplePositionParam.GetAndAdvance());
			ScalarTarget = ScalarTargetParam.GetAndAdvance();
		}
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, Context.GetNumInstances());

		TArray<float>& SampleResults = ExecutionDatas.ScalarResults[(uint8)EFieldCommandResultType::FinalResult];
		SampleResults.Init(0.0, Context.GetNumInstances());

		TArray<float> SampleMax;
		SampleMax.Init(0, Context.GetNumInstances());

		FFieldContext FieldContext{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			InstData->TimeSeconds
		};

		const EFieldPhysicsType PhysicsType = GetFieldTargetTypes(Field_Output_Scalar)[ScalarTarget];
		EvaluateFieldScalarNodes(InstData->FieldCommands, PhysicsType, FieldContext, SampleResults, SampleMax);

		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			OutScalarFieldParam.SetAndAdvance(SampleMax[InstanceIdx]);
		}
	}
	else
	{
		for (int32 InstanceIdx = 0; InstanceIdx < Context.GetNumInstances(); ++InstanceIdx)
		{
			OutScalarFieldParam.SetAndAdvance(0.0);
		}
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfacePhysicsField::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIPhysicsFieldLocal;

	static const TSet<FName> ValidGpuFunctions =
	{
		SamplePhysicsVectorFieldName,
		SamplePhysicsScalarFieldName,
		SamplePhysicsIntegerFieldName,
		EvalPhysicsVectorFieldName,
		EvalPhysicsScalarFieldName,
		EvalPhysicsIntegerFieldName,
		GetPhysicsFieldResolutionName,
		GetPhysicsFieldBoundsName,
	};

	return ValidGpuFunctions.Contains(FunctionInfo.DefinitionName);
}

bool UNiagaraDataInterfacePhysicsField::UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature)
{
	using namespace NDIPhysicsFieldLocal;

	bool bChanged = false;
	
	// upgrade from lwc changes, only parameter types changed there
	if (FunctionSignature.FunctionVersion < FNiagaraPhysicsFieldDIFunctionVersion::LargeWorldCoordinates)
	{
		if (FunctionSignature.Name == SamplePhysicsVectorFieldName && ensure(FunctionSignature.Inputs.Num() == 3))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
		if (FunctionSignature.Name == SamplePhysicsScalarFieldName && ensure(FunctionSignature.Inputs.Num() == 3))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
		if (FunctionSignature.Name == SamplePhysicsIntegerFieldName && ensure(FunctionSignature.Inputs.Num() == 3))
		{
			FunctionSignature.Inputs[1].SetType(FNiagaraTypeDefinition::GetPositionDef());
			bChanged = true;
		}
	}
	FunctionSignature.FunctionVersion = FNiagaraPhysicsFieldDIFunctionVersion::LatestVersion;

	return bChanged;
}

bool UNiagaraDataInterfacePhysicsField::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(NDIPhysicsFieldLocal::TemplateShaderFilePath);
	bSuccess &= InVisitor->UpdateShaderParameters<NDIPhysicsFieldLocal::FShaderInstanceParameters>();
	return bSuccess;
}

void UNiagaraDataInterfacePhysicsField::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, NDIPhysicsFieldLocal::TemplateShaderFilePath, TemplateArgs);
}
#endif

void UNiagaraDataInterfacePhysicsField::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<NDIPhysicsFieldLocal::FShaderInstanceParameters>();
}

void UNiagaraDataInterfacePhysicsField::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	FNDIPhysicsFieldProxy& InterfaceProxy = Context.GetProxy<FNDIPhysicsFieldProxy>();
	FNDIFieldRenderData* ProxyData = InterfaceProxy.SystemInstancesToProxyData.Find(Context.GetSystemInstanceID());

	NDIPhysicsFieldLocal::FShaderInstanceParameters* ShaderInstanceParameters = Context.GetParameterNestedStruct<NDIPhysicsFieldLocal::FShaderInstanceParameters>();

	if (ProxyData != nullptr && ProxyData->FieldResource)
	{
		FPhysicsFieldResource* FieldResource = ProxyData->FieldResource;
		
		// Instance Parameters
		const TArray<EFieldPhysicsType> VectorTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Vector);
		const TArray<EFieldPhysicsType> ScalarTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Scalar);
		const TArray<EFieldPhysicsType> IntegerTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Integer);
		
		ShaderInstanceParameters->NodesParams = FieldResource->NodesParams.SRV;
		ShaderInstanceParameters->NodesOffsets = FieldResource->NodesOffsets.SRV;
		ShaderInstanceParameters->TargetsOffsets = FieldResource->TargetsOffsets.SRV;
		ShaderInstanceParameters->TimeSeconds = ProxyData->TimeSeconds;

		for (int32 i=0; i < MAX_PHYSICS_FIELD_TARGETS; ++i)
		{
			ShaderInstanceParameters->TargetMappings[i].X = i < VectorTypes.Num() ? VectorTypes[i] : 0;
			ShaderInstanceParameters->TargetMappings[i].Y = i < ScalarTypes.Num() ? ScalarTypes[i] : 0;
			ShaderInstanceParameters->TargetMappings[i].Z = i < IntegerTypes.Num() ? IntegerTypes[i] : 0;
			ShaderInstanceParameters->TargetMappings[i].W = 0;

			ShaderInstanceParameters->FieldTargets[i].X = FieldResource->FieldInfos.VectorTargets[i];
			ShaderInstanceParameters->FieldTargets[i].Y = FieldResource->FieldInfos.ScalarTargets[i];
			ShaderInstanceParameters->FieldTargets[i].Z = FieldResource->FieldInfos.IntegerTargets[i];
			ShaderInstanceParameters->FieldTargets[i].W = 0;
		}

		ShaderInstanceParameters->bClipmapAvailable = static_cast<int32>(FieldResource->FieldInfos.bBuildClipmap);
		ShaderInstanceParameters->ClipmapBuffer		= FieldResource->FieldInfos.bBuildClipmap ? FieldResource->ClipmapBuffer.SRV.GetReference() : FNiagaraRenderer::GetDummyFloatBuffer();
		ShaderInstanceParameters->ClipmapCenter		= FVector3f(FieldResource->FieldInfos.ClipmapCenter);
		ShaderInstanceParameters->ClipmapDistance	= FieldResource->FieldInfos.ClipmapDistance;
		ShaderInstanceParameters->ClipmapResolution	= FieldResource->FieldInfos.ClipmapResolution;
		ShaderInstanceParameters->ClipmapExponent	= FieldResource->FieldInfos.ClipmapExponent;
		ShaderInstanceParameters->ClipmapCount		= FieldResource->FieldInfos.ClipmapCount;
		ShaderInstanceParameters->TargetCount		= FieldResource->FieldInfos.TargetCount;
	}
	else
	{
		// Instance Parameters
		ShaderInstanceParameters->NodesParams		= FNiagaraRenderer::GetDummyFloatBuffer();
		ShaderInstanceParameters->NodesOffsets	= FNiagaraRenderer::GetDummyIntBuffer();
		ShaderInstanceParameters->TargetsOffsets	= FNiagaraRenderer::GetDummyIntBuffer();
		ShaderInstanceParameters->TimeSeconds		= 0.0f;
		ShaderInstanceParameters->bClipmapAvailable	= 0;
		ShaderInstanceParameters->ClipmapBuffer		= FNiagaraRenderer::GetDummyFloatBuffer();
		ShaderInstanceParameters->ClipmapCenter		= FVector3f::ZeroVector;
		ShaderInstanceParameters->ClipmapDistance	= 1;
		ShaderInstanceParameters->ClipmapResolution	= 2;
		ShaderInstanceParameters->ClipmapExponent	= 1;
		ShaderInstanceParameters->ClipmapCount		= 1;
		ShaderInstanceParameters->TargetCount		= 0;
		for (int i = 0; i < MAX_PHYSICS_FIELD_TARGETS; ++i)
		{
			ShaderInstanceParameters->TargetMappings[i] = FIntVector4(0, 0, 0, 0);
			ShaderInstanceParameters->FieldTargets[i] = FIntVector4(0, 0, 0, 0);
		}
	}
}

void UNiagaraDataInterfacePhysicsField::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	FNDIPhysicsFieldData* GameThreadData = static_cast<FNDIPhysicsFieldData*>(PerInstanceData);
	FNDIFieldRenderData* RenderThreadData = static_cast<FNDIFieldRenderData*>(DataForRenderThread);

	if (GameThreadData != nullptr && RenderThreadData != nullptr)
	{
		RenderThreadData->FieldResource = GameThreadData->FieldResource;
		RenderThreadData->TimeSeconds = GameThreadData->TimeSeconds;
	}
	check(Proxy);
}

#undef LOCTEXT_NAMESPACE
