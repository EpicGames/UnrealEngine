// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "RHIUtilities.h"
#include "GeometryCollection/GeometryCollectionObject.h"

#include "NiagaraDataInterfaceGeometryCollection.generated.h"

class AGeometryCollectionActor;
struct FNiagaraDataInterfaceGeneratedFunction;


/** Arrays in which the cpu datas will be str */
struct FNDIGeometryCollectionArrays
{
	TArray<FVector4f> WorldTransformBuffer;
	TArray<FVector4f> PrevWorldTransformBuffer;
	TArray<FVector4f> WorldInverseTransformBuffer;
	TArray<FVector4f> PrevWorldInverseTransformBuffer;
	TArray<FVector4f> BoundsBuffer;
	TArray<FTransform> ComponentRestTransformBuffer;
	TArray<uint32> ElementIndexToTransformBufferMapping;

	FNDIGeometryCollectionArrays()
	{
		Resize(100);
	}

	FNDIGeometryCollectionArrays(uint32 Num)
	{
		Resize(Num);
	}

	void CopyFrom(const FNDIGeometryCollectionArrays* Other)
	{
		Resize(Other->NumPieces);
		
		WorldTransformBuffer = Other->WorldTransformBuffer;
		PrevWorldTransformBuffer = Other->PrevWorldTransformBuffer;
		WorldInverseTransformBuffer = Other->WorldInverseTransformBuffer;
		PrevWorldInverseTransformBuffer = Other->PrevWorldInverseTransformBuffer;
		BoundsBuffer = Other->BoundsBuffer;
		ComponentRestTransformBuffer = Other->ComponentRestTransformBuffer;
		ElementIndexToTransformBufferMapping = Other->ElementIndexToTransformBufferMapping;
	}

	void Resize(uint32 Num)
	{		
		NumPieces = Num;

		WorldTransformBuffer.Init(FVector4f(0, 0, 0, 0), 3 * NumPieces);
		PrevWorldTransformBuffer.Init(FVector4f(0, 0, 0, 0), 3 * NumPieces);
		WorldInverseTransformBuffer.Init(FVector4f(0, 0, 0, 0), 3 * NumPieces);
		PrevWorldInverseTransformBuffer.Init(FVector4f(0, 0, 0, 0), 3 * NumPieces);
		BoundsBuffer.Init(FVector4f(0, 0, 0, 0), NumPieces);
		ComponentRestTransformBuffer.Init(FTransform(), 1);
		ElementIndexToTransformBufferMapping.Init(0, NumPieces);
	}

	uint32 NumPieces = 100;
};

/** Render buffers that will be used in hlsl functions */
struct FNDIGeometryCollectionBuffer : public FRenderResource
{
	/** Init the buffer */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	/** Release the buffer */
	virtual void ReleaseRHI() override;

	/** Get the resource name */
	virtual FString GetFriendlyName() const override { return TEXT("FNDIGeometryCollectionBuffer"); }

	/** World transform buffer */
	FReadBuffer WorldTransformBuffer;

	/** Inverse transform buffer*/
	FReadBuffer PrevWorldTransformBuffer;

	/** World transform buffer */
	FReadBuffer WorldInverseTransformBuffer;

	/** Inverse transform buffer*/
	FReadBuffer PrevWorldInverseTransformBuffer;

	/** Element extent buffer */
	FReadBuffer BoundsBuffer;

	/** Per-element transform buffer */
	TRefCountPtr<FRDGPooledBuffer>	ComponentRestTransformBuffer;

	/** Raw data for the transform buffer */
	TArray<uint8> DataToUpload;

	/** number of transforms */
	int32 NumPieces;

	void SetNumPieces(int32 Num)
	{
		NumPieces = Num;
	}
};

struct FResolvedNiagaraGeometryCollection
{
	const UGeometryCollection* GetGeometryCollection() const;
	FTransform GetComponentRootTransform(FNiagaraSystemInstance* SystemInstance) const;
	FTransform GetComponentSpaceTransform(int32 TransformIndex) const;
	TArray<FTransform> GetLocalRestTransforms() const;
	
	TWeakObjectPtr<UGeometryCollection> Collection;
	TWeakObjectPtr<UGeometryCollectionComponent> Component;
};

/** Data stored per physics asset instance*/
struct FNDIGeometryCollectionData
{
	/** Initialize the cpu datas */
	void Init(class UNiagaraDataInterfaceGeometryCollection* Interface, FNiagaraSystemInstance* SystemInstance);

	/** Update the gpu datas */
	void Update(class UNiagaraDataInterfaceGeometryCollection* Interface, FNiagaraSystemInstance* SystemInstance);

	/** Release the buffers */
	void Release();

	ETickingGroup ComputeTickingGroup();

	/** The instance ticking group */
	ETickingGroup TickingGroup;

	/** Actor or geometry collection component world transform, adjusted by lwc system tile */
	FTransform RootTransform;
	
	/** Geometry Collection Bounds */
	FVector3f BoundsOrigin;

	FVector3f BoundsExtent;

	/** Physics asset Gpu buffer */
	FNDIGeometryCollectionBuffer* AssetBuffer = nullptr;

	/** Physics asset Cpu arrays */
	FNDIGeometryCollectionArrays* AssetArrays = nullptr;

	// Flag when there pending transform writes that need to go back to the component
	bool bHasPendingComponentTransformUpdate = false;

	// True if we need to upload new data to the gpu
	bool bNeedsRenderUpdate = false;

	FResolvedNiagaraGeometryCollection ResolvedSource;
};

UENUM()
enum class ENDIGeometryCollection_SourceMode : uint8
{
	/**
	Default behavior follows the order of.
	- Use "Source" when specified (either set explicitly or via blueprint with Set Niagara Geometry Collection Component).
	- Use Parameter Binding if valid
	- Find Geometry Collection Component, Attached Actor, Attached Component
	- Falls back to 'Default Geometry Collection' specified on the data interface
	*/
	Default,

	/**	Only use "Source" (either set explicitly or via blueprint with Set Niagara Geometry Collection Component). */
	Source,

	/**	Only use the parent actor or component the system is attached to. */
	AttachParent,

	/** Only use the "Default Geometry Collection" specified. */
	DefaultCollectionOnly,

	/** Only use the parameter binding. */
	ParameterBinding,
};

/** Data Interface for the Collisions */
UCLASS(EditInlineNew, Category = "Chaos", meta = (DisplayName = "Geometry Collection"), MinimalAPI)
class UNiagaraDataInterfaceGeometryCollection : public UNiagaraDataInterface
{
	GENERATED_UCLASS_BODY()

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER(FVector3f,				BoundsMin)
		SHADER_PARAMETER(FVector3f,				BoundsMax)
		SHADER_PARAMETER(int32,					NumPieces)
		SHADER_PARAMETER(FVector3f,				RootTransform_Translation)
		SHADER_PARAMETER(FQuat4f,				RootTransform_Rotation)
		SHADER_PARAMETER(FVector3f,				RootTransform_Scale)
		SHADER_PARAMETER_SRV(Buffer<float4>,	WorldTransformBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	PrevWorldTransformBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	WorldInverseTransformBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	PrevWorldInverseTransformBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>,	BoundsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer,	ElementTransforms)
	END_SHADER_PARAMETER_STRUCT()

	/** Controls how to retrieve the Skeletal Mesh Component to attach to. */
	UPROPERTY(EditAnywhere, Category = "Geometry Collection")
	ENDIGeometryCollection_SourceMode SourceMode = ENDIGeometryCollection_SourceMode::Default;

#if WITH_EDITORONLY_DATA
	/** Geometry collection used to sample from when not overridden by a source actor from the scene. Only available in editor for previewing. This is removed in cooked builds. */
	UPROPERTY(EditAnywhere, Category = "Geometry Collection")
	TSoftObjectPtr<UGeometryCollection> PreviewCollection;
#endif

	/** GeometryCollection used to sample from when not overridden by a source actor from the scene. This reference is NOT removed from cooked builds. */
	UPROPERTY(EditAnywhere, Category = "Geometry Collection", meta = (EditConditionHides, EditCondition = "SourceMode == ENDIGeometryCollection_SourceMode::Default || SourceMode == ENDIGeometryCollection_SourceMode::DefaultCollectionOnly"))
	TObjectPtr<UGeometryCollection> DefaultGeometryCollection;

	/** The source actor from which to sample. Takes precedence over the direct geometry collection. Note that this can only be set when used as a user variable on a niagara component in the world.*/
	UPROPERTY(EditAnywhere, Category = "Geometry Collection", meta = (DisplayName = "Source Actor", EditConditionHides, EditCondition = "SourceMode == ENDIGeometryCollection_SourceMode::Default || SourceMode == ENDIGeometryCollection_SourceMode::Source"))
	TSoftObjectPtr<AGeometryCollectionActor> GeometryCollectionActor;

	/** The source component from which to sample. Takes precedence over the direct mesh. Not exposed to the user, only indirectly accessible from blueprints. */
	UPROPERTY(Transient)
	TObjectPtr<UGeometryCollectionComponent> SourceComponent;

	/** Reference to a user parameter if we're reading one. */
	UPROPERTY(EditAnywhere, Category = "Geometry Collection", meta = (EditConditionHides, EditCondition = "SourceMode == ENDIGeometryCollection_SourceMode::Default || SourceMode == ENDIGeometryCollection_SourceMode::ParameterBinding"))
	FNiagaraUserParameterBinding GeometryCollectionUserParameter;
	
	// If true then this data interface will also read and write intermediate bones or geometry, otherwise only leaf nodes are considered
	UPROPERTY(EditAnywhere, Category = "Geometry Collection")
	bool bIncludeIntermediateBones = false;

	/** UObject Interface */
	virtual void PostInitProperties() override;

	/** UNiagaraDataInterface Interface */
	virtual void GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc) override;
	virtual bool CanExecuteOnTarget(ENiagaraSimTarget Target) const override { return true; }
	virtual bool InitPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance) override;
	virtual void DestroyPerInstanceData(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance)override;
	virtual bool PerInstanceTick(void* PerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds) override;
	virtual int32 PerInstanceDataSize()const override { return sizeof(FNDIGeometryCollectionData); }
	virtual bool PerInstanceTickPostSimulate(void* PerInstanceData, FNiagaraSystemInstance* InSystemInstance, float DeltaSeconds) override;
	virtual bool Equals(const UNiagaraDataInterface* Other) const override;
	virtual bool HasPreSimulateTick() const override { return true; }
	virtual bool HasPostSimulateTick() const override { return true; }
	virtual bool PostSimulateCanOverlapFrames() const override { return false; }
	virtual bool HasTickGroupPrereqs() const override { return true; }
	virtual ETickingGroup CalculateTickGroup(const void* PerInstanceData) const override;


	/** GPU simulation  functionality */
#if WITH_EDITORONLY_DATA
	virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	virtual void GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL) override;
	virtual bool GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL) override;
	virtual bool UpgradeFunctionCall(FNiagaraFunctionSignature& FunctionSignature) override;
#endif
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const override;
	virtual void SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const override;

	virtual void ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance) override;

private:
	// VM functions
	void GetNumGeometryElements(FVectorVMExternalFunctionContext& Context);
	void GetElementBounds(FVectorVMExternalFunctionContext& Context);
	void GetElementTransformCS(FVectorVMExternalFunctionContext& Context);
	void SetElementTransformCS(FVectorVMExternalFunctionContext& Context);
	void SetElementTransformWS(FVectorVMExternalFunctionContext& Context);
	void GetActorTransform(FVectorVMExternalFunctionContext& Context);

protected:
#if WITH_EDITORONLY_DATA
	virtual void GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const override;
#endif

	/** Copy one niagara DI to this */
	virtual bool CopyToInternal(UNiagaraDataInterface* Destination) const override;

private:
	void ResolveGeometryCollection(FNiagaraSystemInstance* SystemInstance, FNDIGeometryCollectionData* InstanceData);
	bool ResolveGeometryCollectionFromDirectSource(FResolvedNiagaraGeometryCollection& ResolvedSource);
	bool ResolveGeometryCollectionFromAttachParent(FNiagaraSystemInstance* SystemInstance, FResolvedNiagaraGeometryCollection& ResolvedSource);
	bool ResolveGeometryCollectionFromActor(AActor* Actor, FResolvedNiagaraGeometryCollection& ResolvedSource);
	bool ResolveGeometryCollectionFromDefaultCollection(FResolvedNiagaraGeometryCollection& ResolvedSource);
	bool ResolveGeometryCollectionFromParameterBinding(UObject* UserParameter, FResolvedNiagaraGeometryCollection& ResolvedSource);
};

/** Proxy to send data to gpu */
struct FNDIGeometryCollectionProxy : public FNiagaraDataInterfaceProxy
{
	/** Get the size of the data that will be passed to render*/
	virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FNDIGeometryCollectionData); }

	/** Get the data that will be passed to render*/
	virtual void ConsumePerInstanceDataFromGameThread(void* PerInstanceData, const FNiagaraSystemInstanceID& Instance) override;

	/** Initialize the Proxy data buffer */
	void InitializePerInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	/** Destroy the proxy data if necessary */
	void DestroyPerInstanceData(const FNiagaraSystemInstanceID& SystemInstance);

	/** Launch all pre stage functions */
	virtual void PreStage(const FNDIGpuComputePreStageContext& Context) override;

	/** List of proxy data for each system instances*/
	TMap<FNiagaraSystemInstanceID, FNDIGeometryCollectionData> SystemInstancesToProxyData;
};
